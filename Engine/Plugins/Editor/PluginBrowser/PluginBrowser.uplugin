{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Plugin Browser",
	"Description" : "User interface for managing installed plugins and creating new ones.",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc. and Karol Szyszka",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "PluginBrowser",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	],
	"Plugins": [ 
		{ 
			"Name": "PluginUtils",
			"Enabled": true 
		} 
	]
}