// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PluginBrowser : ModuleRules
	{
		public PluginBrowser(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"UnrealEd"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",		// @todo Mac: for some reason CoreUObject and Engine are needed to link in debug on Mac
                    "InputCore",
					"Engine",
					"Slate",
					"SlateCore",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"PluginUtils",
					"Projects",
					"EditorFramework",
					"UnrealEd",
					"PropertyEditor",
					"SharedSettingsWidgets",
					"DirectoryWatcher",
					"GameProjectGeneration",
					"MainFrame",
                    "UATHelper",
                    "AssetTools",
					"Json",
					"ToolWidgets",
					"ToolMenus",
					"EditorWidgets",
					"ContentBrowser",
					"ContentBrowserData",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"DesktopPlatform",
				}
			);

			// TODO: Move back to using this if we can remove dependencies on GameProjectUtils
			/*DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"GameProjectGeneration",
				}
			);*/
		}
	}
}
