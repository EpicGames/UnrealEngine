// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultPluginWizardDefinition.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Interfaces/IPluginManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "PluginBrowserModule.h"

#define LOCTEXT_NAMESPACE "NewPluginWizard"

FDefaultPluginWizardDefinition::FDefaultPluginWizardDefinition(bool bContentOnlyProject)
	: bIsContentOnlyProject(bContentOnlyProject)
{
	PluginBaseDir = IPluginManager::Get().FindPlugin(TEXT("PluginBrowser"))->GetBaseDir();

	PopulateTemplatesSource();
}

void FDefaultPluginWizardDefinition::PopulateTemplatesSource()
{
	TemplateDefinitions = FPluginBrowserModule::Get().GetDefaultPluginTemplates();

	// Add external templates that came from the modular feature interface (e.g., from another plugin like Game Features)
	TemplateDefinitions.Append(FPluginBrowserModule::Get().GetAddedPluginTemplates());

	// Don't show the option to make an engine plugin in installed builds
	const bool bAllowEnginePlugins = !FApp::IsEngineInstalled();
	for (const TSharedRef<FPluginTemplateDescription>& Template : TemplateDefinitions)
	{
		Template->bCanBePlacedInEngine = Template->bCanBePlacedInEngine && bAllowEnginePlugins;
	}

	TemplateDefinitions.Sort([](const TSharedRef<FPluginTemplateDescription>& A, const TSharedRef<FPluginTemplateDescription>& B)
	{
		if (A->SortPriority != B->SortPriority)
		{
			return A->SortPriority > B->SortPriority;
		}
		else
		{
			return A->Name.CompareTo(B->Name) <= 0;
		}
	});
}

const TArray<TSharedRef<FPluginTemplateDescription>>& FDefaultPluginWizardDefinition::GetTemplatesSource() const
{
	return TemplateDefinitions;
}


void FDefaultPluginWizardDefinition::OnTemplateSelectionChanged(TSharedPtr<FPluginTemplateDescription> InSelectedItem, ESelectInfo::Type SelectInfo)
{
	CurrentTemplateDefinition = InSelectedItem;
}

TSharedPtr<FPluginTemplateDescription> FDefaultPluginWizardDefinition::GetSelectedTemplate() const
{
	return CurrentTemplateDefinition;
}

bool FDefaultPluginWizardDefinition::HasValidTemplateSelection() const
{
	return CurrentTemplateDefinition.IsValid();
}

void FDefaultPluginWizardDefinition::ClearTemplateSelection()
{
	CurrentTemplateDefinition.Reset();
}

bool FDefaultPluginWizardDefinition::HasModules() const
{
	FString SourceFolderPath = GetPluginFolderPath() / TEXT("Source");
	
	return FPaths::DirectoryExists(SourceFolderPath);
}

bool FDefaultPluginWizardDefinition::IsMod() const
{
	return false;
}

FText FDefaultPluginWizardDefinition::GetInstructions() const
{
	return LOCTEXT("ChoosePluginTemplate", "Choose a template and then specify a name to create a new plugin.");
}

bool FDefaultPluginWizardDefinition::GetPluginIconPath(FString& OutIconPath) const
{
	return GetTemplateIconPath(CurrentTemplateDefinition.ToSharedRef(), OutIconPath);
}

EHostType::Type FDefaultPluginWizardDefinition::GetPluginModuleDescriptor() const
{
	EHostType::Type ModuleDescriptorType = EHostType::Runtime;

	if (CurrentTemplateDefinition.IsValid())
	{
		ModuleDescriptorType = CurrentTemplateDefinition->ModuleDescriptorType;
	}

	return ModuleDescriptorType;
}

ELoadingPhase::Type FDefaultPluginWizardDefinition::GetPluginLoadingPhase() const
{
	ELoadingPhase::Type Phase = ELoadingPhase::Default;

	if (CurrentTemplateDefinition.IsValid())
	{
		Phase = CurrentTemplateDefinition->LoadingPhase;
	}

	return Phase;
}

bool FDefaultPluginWizardDefinition::GetTemplateIconPath(TSharedRef<FPluginTemplateDescription> Template, FString& OutIconPath) const
{
	bool bRequiresDefaultIcon = false;

	FString TemplateFolderName = GetFolderForTemplate(Template);

	OutIconPath = TemplateFolderName / TEXT("Resources/Icon128.png");
	if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*OutIconPath))
	{
		OutIconPath = PluginBaseDir / TEXT("Resources/DefaultIcon128.png");
		bRequiresDefaultIcon = true;
	}

	return bRequiresDefaultIcon;
}

TArray<FString> FDefaultPluginWizardDefinition::GetFoldersForSelection() const
{
	TArray<FString> SelectedFolders;

	if (CurrentTemplateDefinition.IsValid())
	{
		SelectedFolders.Add(GetFolderForTemplate(CurrentTemplateDefinition.ToSharedRef()));
	}

	return SelectedFolders;
}

void FDefaultPluginWizardDefinition::PluginCreated(const FString& PluginName, bool bWasSuccessful) const
{
}

FString FDefaultPluginWizardDefinition::GetPluginFolderPath() const
{
	return GetFolderForTemplate(CurrentTemplateDefinition.ToSharedRef());
}

FString FDefaultPluginWizardDefinition::GetFolderForTemplate(TSharedRef<FPluginTemplateDescription> InTemplate) const
{
	return InTemplate->OnDiskPath;
}

#undef LOCTEXT_NAMESPACE
