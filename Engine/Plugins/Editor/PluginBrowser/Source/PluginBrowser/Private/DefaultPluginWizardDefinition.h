// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPluginWizardDefinition.h"
#include "Styling/SlateTypes.h"

class FDefaultPluginWizardDefinition : public IPluginWizardDefinition
{
public:
	FDefaultPluginWizardDefinition(bool bIsContentOnlyProject = false);

	// Begin IPluginWizardDefinition interface
	virtual const TArray<TSharedRef<FPluginTemplateDescription>>& GetTemplatesSource() const override;
	virtual void OnTemplateSelectionChanged(TSharedPtr<FPluginTemplateDescription> InSelectedItem, ESelectInfo::Type SelectInfo) override;
	virtual TSharedPtr<FPluginTemplateDescription> GetSelectedTemplate() const override;
	virtual void ClearTemplateSelection() override;
	virtual bool HasValidTemplateSelection() const override;

	virtual bool CanShowOnStartup() const override { return false; }
	virtual bool HasModules() const override;
	virtual bool IsMod() const override;
	virtual void OnShowOnStartupCheckboxChanged(ECheckBoxState CheckBoxState) override {}
	virtual ECheckBoxState GetShowOnStartupCheckBoxState() const override { return ECheckBoxState::Undetermined; }
	virtual TSharedPtr<class SWidget> GetCustomHeaderWidget() override { return nullptr; }
	virtual FText GetInstructions() const override;

	virtual bool GetPluginIconPath(FString& OutIconPath) const override;
	virtual EHostType::Type GetPluginModuleDescriptor() const override;
	virtual ELoadingPhase::Type GetPluginLoadingPhase() const override;
	virtual bool GetTemplateIconPath(TSharedRef<FPluginTemplateDescription> InTemplate, FString& OutIconPath) const override;
	virtual FString GetPluginFolderPath() const override;
	virtual TArray<FString> GetFoldersForSelection() const override;
	virtual void PluginCreated(const FString& PluginName, bool bWasSuccessful) const override;
	// End IPluginWizardDefinition interface

private:
	/** Creates the templates that can be used by the plugin manager to generate the plugin */
	void PopulateTemplatesSource();

	/** Gets the folder for the specified template. */
	FString GetFolderForTemplate(TSharedRef<FPluginTemplateDescription> InTemplate) const;

private:
	/** The templates available to this definition */
	TArray<TSharedRef<FPluginTemplateDescription>> TemplateDefinitions;

	/** The currently selected template definition */
	TSharedPtr<FPluginTemplateDescription> CurrentTemplateDefinition;

	/** Base directory of the plugin templates */
	FString PluginBaseDir;

	/** If true, this definition is for a project that can only contain content */
	bool bIsContentOnlyProject;
};
