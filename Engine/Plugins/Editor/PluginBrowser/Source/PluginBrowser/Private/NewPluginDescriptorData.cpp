// Copyright Epic Games, Inc. All Rights Reserved.

#include "NewPluginDescriptorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NewPluginDescriptorData)

UNewPluginDescriptorData::UNewPluginDescriptorData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsBetaVersion(false)
{
}
