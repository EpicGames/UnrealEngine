// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NewPluginDescriptorData.generated.h"

/**
 * An object used internally by the New Plugin Wizard to set user-defined plugin descriptor information.
 * This should not be used outside of the New Plugin Wizard dialog
 */
UCLASS()
class UNewPluginDescriptorData : public UObject
{
	GENERATED_BODY()

public:

	UNewPluginDescriptorData(const FObjectInitializer& ObjectInitializer);

	/** The author of this content */
	UPROPERTY(EditAnywhere, Category="Descriptor Data", meta=(DisplayName="Author"))
	FString CreatedBy;

	/** Optional hyperlink for the author's website  */
	UPROPERTY(EditAnywhere, Category="Descriptor Data", AdvancedDisplay, meta=(DisplayName="Author URL"))
	FString CreatedByURL;

	/** A description for this content */
	UPROPERTY(EditAnywhere, Category="Descriptor Data")
	FString Description;

	/** Marks this content as being in beta */
	UPROPERTY(EditAnywhere, Category="Descriptor Data", AdvancedDisplay)
	bool bIsBetaVersion;

};
