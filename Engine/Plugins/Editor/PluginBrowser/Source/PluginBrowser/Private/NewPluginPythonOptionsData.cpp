// Copyright Epic Games, Inc. All Rights Reserved.


#include "NewPluginPythonOptionsData.h"

UNewPluginPythonOptionsData::UNewPluginPythonOptionsData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}