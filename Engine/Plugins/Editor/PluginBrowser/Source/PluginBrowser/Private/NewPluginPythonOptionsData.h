// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "NewPluginPythonOptionsData.generated.h"

/**
 * An object used internally by the New Plugin Wizard to set the post-create python script options.
 * This should not be used outside of the New Plugin Wizard dialog
 */
UCLASS()
class UNewPluginPythonOptionsData : public UObject
{
	GENERATED_BODY()
	
public:
	UNewPluginPythonOptionsData(const FObjectInitializer& ObjectInitializer);

	/** Optional additional arguments for the python post-create script  */
	UPROPERTY(EditAnywhere, Category="Post Create Python Script")
	FString AdditionalArguments;
};
