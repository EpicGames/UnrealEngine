// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginActions.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "Interfaces/IPluginManager.h"
#include "IUATHelperModule.h"

#define LOCTEXT_NAMESPACE "PluginListTile"

void PluginActions::PackagePlugin(TSharedRef<IPlugin> Plugin, TSharedPtr<SWidget> ParentWidget)
{
	FString DefaultDirectory;
	FString OutputDirectory;

	if ( !FDesktopPlatformModule::Get()->OpenDirectoryDialog(FSlateApplication::Get().FindBestParentWindowHandleForDialogs(ParentWidget), LOCTEXT("PackagePluginDialogTitle", "Package Plugin...").ToString(), DefaultDirectory, OutputDirectory) )
	{
		return;
	}

	// Ensure path is full rather than relative (for macs)
	FString DescriptorFilename = Plugin->GetDescriptorFileName();
	FString DescriptorFullPath = FPaths::ConvertRelativePathToFull(DescriptorFilename);
	OutputDirectory = FPaths::Combine(OutputDirectory, Plugin->GetName());
	FString CommandLine = FString::Printf(TEXT("BuildPlugin -Plugin=\"%s\" -Package=\"%s\" -CreateSubFolder"), *DescriptorFullPath, *OutputDirectory);

#if PLATFORM_WINDOWS
	FText PlatformName = LOCTEXT("PlatformName_Windows", "Windows");
#elif PLATFORM_MAC
	FText PlatformName = LOCTEXT("PlatformName_Mac", "Mac");
#elif PLATFORM_LINUX
	FText PlatformName = LOCTEXT("PlatformName_Linux", "Linux");
#else
	FText PlatformName = LOCTEXT("PlatformName_Other", "Other OS");
#endif

	IUATHelperModule::Get().CreateUatTask(CommandLine, PlatformName, LOCTEXT("PackagePluginTaskName", "Packaging Plugin"),
		LOCTEXT("PackagePluginTaskShortName", "Package Plugin Task"), FAppStyle::GetBrush(TEXT("MainFrame.CookContent")));
}

#undef LOCTEXT_NAMESPACE