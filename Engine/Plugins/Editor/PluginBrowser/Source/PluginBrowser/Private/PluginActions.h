// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IPlugin; 
class SWidget; 

namespace PluginActions
{
    void PackagePlugin(TSharedRef<IPlugin> Plugin, TSharedPtr<SWidget> ParentWidget);
}