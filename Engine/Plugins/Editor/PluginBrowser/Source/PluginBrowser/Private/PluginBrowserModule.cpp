// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginBrowserModule.h"

#include "Algo/AnyOf.h"
#include "ContentBrowserModule.h"
#include "Features/IModularFeatures.h"
#include "Features/EditorFeatures.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/NamePermissionList.h"
#include "Misc/PathViews.h"
#include "PropertyEditorModule.h"
#include "PluginActions.h"
#include "PluginDescriptorEditor.h"
#include "PluginMetadataObject.h"
#include "PluginStyle.h"
#include "SNewPluginWizard.h"
#include "SPluginBrowser.h"
#include "SPluginPaths.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UnrealEdMisc.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "PluginsEditor"

LLM_DEFINE_TAG(PluginBrowser);

DEFINE_LOG_CATEGORY(LogPluginBrowser);

IMPLEMENT_MODULE( FPluginBrowserModule, PluginBrowser )

const FName FPluginBrowserModule::PluginsEditorTabName( TEXT( "PluginsEditor" ) );
const FName FPluginBrowserModule::PluginCreatorTabName( TEXT( "PluginCreator" ) );
const FName FPluginBrowserModule::ExternalDirectoriesTabName(TEXT("PluginDirectories"));

void FPluginBrowserModule::StartupModule()
{
	LLM_SCOPE_BYTAG(PluginBrowser);

	FPluginStyle::Initialize();

	// Register ourselves as an editor feature
	IModularFeatures::Get().RegisterModularFeature( EditorFeatures::PluginsEditor, this );

	// Register the detail customization for the metadata object
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(UPluginMetadataObject::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPluginMetadataCustomization::MakeInstance));
	
	// Register a tab spawner so that our tab can be automatically restored from layout files
	FGlobalTabmanager::Get()->RegisterTabSpawner( PluginsEditorTabName, FOnSpawnTab::CreateRaw(this, &FPluginBrowserModule::HandleSpawnPluginBrowserTab ) )
			.SetDisplayName( LOCTEXT( "PluginsEditorTabTitle", "Plugins" ) )
			.SetTooltipText( LOCTEXT( "PluginsEditorTooltipText", "Open the Plugins Browser tab." ) )
			.SetIcon(FSlateIcon(FPluginStyle::Get()->GetStyleSetName(), "Plugins.TabIcon"));

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		PluginCreatorTabName,
		FOnSpawnTab::CreateRaw(this, &FPluginBrowserModule::HandleSpawnPluginCreatorTab))
		.SetDisplayName(LOCTEXT("NewPluginTabHeader", "New Plugin"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	// Register a default size for this tab
	const FVector2D DefaultCreatorSize(1000.0f, 750.0f);
	FTabManager::RegisterDefaultTabWindowSize(PluginCreatorTabName, DefaultCreatorSize);

	// Additional Plugin Directories tab
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ExternalDirectoriesTabName, FOnSpawnTab::CreateRaw(this, &FPluginBrowserModule::HandleSpawnExternalDirectoriesTab))
		.SetDisplayName(LOCTEXT("ExternalPluginDirectoriesTabTitle", "Plugin Directories"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FPluginStyle::Get()->GetStyleSetName(), "Plugins.TabIcon"));

	const FVector2D DefaultDirectoriesSize(750.0f, 600.0f);
	FTabManager::RegisterDefaultTabWindowSize(ExternalDirectoriesTabName, DefaultDirectoriesSize);

	// Get a list of the installed plugins we've seen before
	TArray<FString> PreviousInstalledPlugins;
	GConfig->GetArray(TEXT("PluginBrowser"), TEXT("InstalledPlugins"), PreviousInstalledPlugins, GEditorPerProjectIni);

	// Find all the plugins that are installed
	for(const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		if(Plugin->GetDescriptor().bInstalled)
		{
			InstalledPlugins.Add(Plugin->GetName());
		}
	}

	// Find all the plugins which have been newly installed
	NewlyInstalledPlugins.Reset();
	for(const FString& InstalledPlugin : InstalledPlugins)
	{
		if(!PreviousInstalledPlugins.Contains(InstalledPlugin))
		{
			NewlyInstalledPlugins.Add(InstalledPlugin);
		}
	}

	// Register a callback to check for new plugins on startup
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	MainFrameModule.OnMainFrameCreationFinished().AddRaw(this, &FPluginBrowserModule::OnMainFrameLoaded);
	
	AddContentBrowserMenuExtensions();
	GenerateDefaultPluginTemplateDescriptions();

	IPluginManager::Get().GetExternalPluginSources(OriginalExternalSources);
}

void FPluginBrowserModule::ShutdownModule()
{
	FPluginStyle::Shutdown();

	// Unregister the main frame callback
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameCreationFinished().RemoveAll(this);
	}

	// Unregister the tab spawner
	FGlobalTabmanager::Get()->UnregisterTabSpawner( PluginsEditorTabName );
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner( PluginCreatorTabName );
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ExternalDirectoriesTabName);

	// Unregister our feature
	IModularFeatures::Get().UnregisterModularFeature( EditorFeatures::PluginsEditor, this );
}

static void CreatePluginMenu(UToolMenu* Menu, TSharedRef<IPlugin> Plugin)
{
	UContentBrowserDataMenuContext_FolderMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FolderMenu>();
	
	FToolMenuSection& Section = Menu->FindOrAddSection("PluginMenu");
	Section.AddMenuEntry(
		"EditPluginProperties",
		LOCTEXT("PluginContextMenu.EditPlugin", "Edit"),
		LOCTEXT("PluginContextMenu.EditPlugin.ToolTip", "Edit this plugin's properties."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
		FExecuteAction::CreateLambda([Plugin](){
			FPluginBrowserModule::Get().OpenPluginEditor(Plugin, {}, {});
		})	
	);

	Section.AddMenuEntry(
		"PackagePlugin",
		LOCTEXT("PluginContextMenu.PackagePlugin", "Package"),
		LOCTEXT("PluginContextMenu.PackagePlugin.ToolTip", "Package this plugin for distribution"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Package"),
		FExecuteAction::CreateLambda([Plugin](){
			PluginActions::PackagePlugin(Plugin, {});
		})	
	);

	if (FPluginBrowserModule::Get().OnLaunchReferenceViewerDelegate().IsBound())
	{
		Section.AddMenuEntry(
			"PluginReferenceViewer",
			LOCTEXT("PluginContextMenu.ReferenceViewer", "Reference Viewer"),
			LOCTEXT("PluginContextMenu.ReferenceViewer.ToolTip", "Open the plugin reference viewer showing references to and from this plugin"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.ReferenceViewer"),
			FExecuteAction::CreateLambda([Plugin](){
				FPluginBrowserModule::Get().OnLaunchReferenceViewerDelegate().Execute(Plugin);
			})	
		);
	}

	const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
	if (Descriptor.SupportURL.StartsWith(TEXT("http://")))
	{
		Section.AddMenuEntry(
			"PluginSupport",
			LOCTEXT("PluginContextMenu.Support", "Support"),
			LOCTEXT("PluginContextMenu.Support.ToolTip", "Open the plugin's online support URL"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Comment"),
			FExecuteAction::CreateLambda([Plugin](){
				const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
				if (Descriptor.SupportURL.StartsWith(TEXT("http://")))
				{
					FPlatformProcess::LaunchURL(*Descriptor.SupportURL, nullptr, nullptr);
				}
			})	
		);
	}
	
	if (Descriptor.DocsURL.StartsWith(TEXT("http://")))
	{	
		Section.AddMenuEntry(
			"PluginDocumentation",
			LOCTEXT("PluginContextMenu.Documentation", "Documentation"),
			LOCTEXT("PluginContextMenu.Documentation.ToolTip", "Open the plugin's online support URL."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation"),
			FExecuteAction::CreateLambda([Plugin](){
				const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
				if (Descriptor.DocsURL.StartsWith(TEXT("http://")))
				{
					FPlatformProcess::LaunchURL(*Descriptor.DocsURL, nullptr, nullptr);
				}
			})	
		);
	}
	
	if (Descriptor.CreatedBy.IsEmpty() == false && Descriptor.CreatedByURL.StartsWith(TEXT("http://")))
	{
		Section.AddMenuEntry(
			"PluginVendor",
			LOCTEXT("PluginContextMenu.Vendor", "Vendor"),
			LOCTEXT("PluginContextMenu.Vendor.ToolTip", "Visit the plugin vendor's website."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.OpenInBrowser"),
			FExecuteAction::CreateLambda([Plugin](){
				const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
				if (!Descriptor.CreatedBy.IsEmpty() && Descriptor.CreatedByURL.StartsWith(TEXT("http://")))
				{
					FPlatformProcess::LaunchURL(*Descriptor.CreatedByURL, nullptr, nullptr);
				}
			})	
		);
	}
}

void AddPluginMenuToContextMenu(UToolMenu* InMenu, const TSharedPtr<IPlugin>& Plugin, FName AddToSection)
{
	FToolMenuSection &Section = InMenu->FindOrAddSection(AddToSection);
	Section.AddSubMenu("Plugin",
	   LOCTEXT("ContentBrowserPluginMenuName", "Plugin"),
	   LOCTEXT("ContentBrowserPluginMenuTooltip", "Plugin Actions"),
	   FNewToolMenuDelegate::CreateStatic(&CreatePluginMenu, Plugin.ToSharedRef()),
	   false,
	   FSlateIcon(FPluginStyle::Get()->GetStyleSetName(), "Plugins.TabIcon"));
}

void FPluginBrowserModule::AddContentBrowserMenuExtensions()
{
	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu"))
	{
		FToolMenuSection& Section = Menu->AddDynamicSection(TEXT("DynamicSection_Plugins"), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			UContentBrowserDataMenuContext_FolderMenu* Context = InMenu->FindContext<UContentBrowserDataMenuContext_FolderMenu>();
			// Only show the menu if there is a single item selected as most commands don't work nicely for multiple plugins 
			if (Context->SelectedItems.Num() == 1 && Context->SelectedItems[0].IsInPlugin())
			{
				FNameBuilder ItemPath{Context->SelectedItems[0].GetInternalPath()};
				FStringView PluginName = FPathViews::GetMountPointNameFromPath(ItemPath.ToView());
				if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
				{
					AddPluginMenuToContextMenu(InMenu, Plugin, "Section");
				}
			}
		}));
	}
	
	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu"))
	{
		FToolMenuSection& Section = Menu->AddDynamicSection(TEXT("DynamicSection_Plugins"), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			UContentBrowserDataMenuContext_FileMenu* Context = InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
			// Only show the menu if there is a single item selected as most commands don't work nicely for multiple plugins 
			if (Context->SelectedItems.Num() == 1 && Context->SelectedItems[0].IsInPlugin())
			{
				FNameBuilder ItemPath{Context->SelectedItems[0].GetInternalPath()};
				FStringView PluginName = FPathViews::GetMountPointNameFromPath(ItemPath.ToView());
				if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
				{
					AddPluginMenuToContextMenu(InMenu, Plugin, "AssetContextExploreMenuOptions");
				}
			}
		}));
	}
}

void FPluginBrowserModule::GenerateDefaultPluginTemplateDescriptions()
{
	DefaultPluginTemplates.Reset(7);
	const FString PluginBaseDir = IPluginManager::Get().FindPlugin(TEXT("PluginBrowser"))->GetBaseDir();

	const FText BlankTemplateName = LOCTEXT("BlankLabel", "Blank");
	const FText ContentOnlyTemplateName = LOCTEXT("ContentOnlyLabel", "Content Only");
	const FText BasicTemplateName = LOCTEXT("BasicTemplateTabLabel", "Editor Toolbar Button");
	const FText AdvancedTemplateName = LOCTEXT("AdvancedTemplateTabLabel", "Editor Standalone Window");
	const FText BlueprintLibTemplateName = LOCTEXT("BlueprintLibTemplateLabel", "Blueprint Library");
	const FText EditorModeTemplateName = LOCTEXT("EditorModeTemplateLabel", "Editor Mode");
	const FText ThirdPartyTemplateName = LOCTEXT("ThirdPartyTemplateLabel", "Third Party Library");

	const FText BlankDescription = LOCTEXT("BlankTemplateDesc", "Create a blank plugin with a minimal amount of code.\n\nChoose this if you want to set everything up from scratch or are making a non-visual plugin.\nA plugin created with this template will appear in the Editor's plugin list but will not register any buttons or menu entries.");
	const FText ContentOnlyDescription = LOCTEXT("ContentOnlyTemplateDesc", "Create a blank plugin that can only contain content.");
	const FText BasicDescription = LOCTEXT("BasicTemplateDesc", "Create a plugin that will add a button to the toolbar in the Level Editor.\n\nStart by implementing something in the created \"OnButtonClick\" event.");
	const FText AdvancedDescription = LOCTEXT("AdvancedTemplateDesc", "Create a plugin that will add a button to the toolbar in the Level Editor that summons an empty standalone tab window when clicked.");
	const FText BlueprintLibDescription = LOCTEXT("BPLibTemplateDesc", "Create a plugin that will contain Blueprint Function Library.\n\nChoose this if you want to create static blueprint nodes.");
	const FText EditorModeDescription = LOCTEXT("EditorModeDesc", "Create a plugin that will have an editor mode.\n\nThis will include a toolkit example to specify UI that will appear in \"Modes\" tab (next to Foliage, Landscape etc).\nIt will also include very basic UI that demonstrates editor interaction and undo/redo functions usage.");
	const FText ThirdPartyDescription = LOCTEXT("ThirdPartyDesc", "Create a plugin that uses an included third party library.\n\nThis can be used as an example of how to include, load and use a third party library yourself.");

	TSharedRef<FPluginTemplateDescription> ContentOnlyTemplate = MakeShareable(new FPluginTemplateDescription(ContentOnlyTemplateName, ContentOnlyDescription, PluginBaseDir / TEXT("Templates") / TEXT("ContentOnly"), true, EHostType::Runtime));
	ContentOnlyTemplate->SortPriority = 1;
	DefaultPluginTemplates.Add(ContentOnlyTemplate);

	// Insert the blank template to make sure it appears before the content only template.

	TSharedRef<FPluginTemplateDescription> BlankTemplate = MakeShareable(new FPluginTemplateDescription(BlankTemplateName, BlankDescription, PluginBaseDir / TEXT("Templates") / TEXT("Blank"), true, EHostType::Runtime, ELoadingPhase::Default, false));
	BlankTemplate->SortPriority = 2;
	DefaultPluginTemplates.Add(BlankTemplate);

	DefaultPluginTemplates.Add(MakeShareable(new FPluginTemplateDescription(BlueprintLibTemplateName, BlueprintLibDescription, PluginBaseDir / TEXT("Templates") / TEXT("BlueprintLibrary"), true, EHostType::Runtime, ELoadingPhase::PreLoadingScreen, false)));
	DefaultPluginTemplates.Add(MakeShareable(new FPluginTemplateDescription(BasicTemplateName, BasicDescription, PluginBaseDir / TEXT("Templates") / TEXT("Basic"), false, EHostType::Editor, ELoadingPhase::Default, false)));
	DefaultPluginTemplates.Add(MakeShareable(new FPluginTemplateDescription(AdvancedTemplateName, AdvancedDescription, PluginBaseDir / TEXT("Templates") / TEXT("Advanced"), false, EHostType::Editor, ELoadingPhase::Default, false)));
	DefaultPluginTemplates.Add(MakeShareable(new FPluginTemplateDescription(EditorModeTemplateName, EditorModeDescription, PluginBaseDir / TEXT("Templates") / TEXT("EditorMode"), false, EHostType::Editor, ELoadingPhase::Default, false)));
	DefaultPluginTemplates.Add(MakeShareable(new FPluginTemplateDescription(ThirdPartyTemplateName, ThirdPartyDescription, PluginBaseDir / TEXT("Templates") / TEXT("ThirdPartyLibrary"), true, EHostType::Runtime, ELoadingPhase::Default, false)));
}

void FPluginBrowserModule::RegisterPluginTemplate(TSharedRef<FPluginTemplateDescription> Template)
{
	LLM_SCOPE_BYTAG(PluginBrowser);
	AddedPluginTemplates.Add(Template);
}

void FPluginBrowserModule::UnregisterPluginTemplate(TSharedRef<FPluginTemplateDescription> Template)
{
	AddedPluginTemplates.RemoveSingle(Template);
}

FPluginEditorExtensionHandle FPluginBrowserModule::RegisterPluginEditorExtension(FOnPluginBeingEdited Extension)
{
	LLM_SCOPE_BYTAG(PluginBrowser);
	++EditorExtensionCounter;
	FPluginEditorExtensionHandle Result = EditorExtensionCounter;
	CustomizePluginEditingDelegates.Add(MakeTuple(Extension, Result));
	return Result;
}

void FPluginBrowserModule::UnregisterPluginEditorExtension(FPluginEditorExtensionHandle ExtensionHandle)
{
	CustomizePluginEditingDelegates.RemoveAll([=](TPair<FOnPluginBeingEdited, FPluginEditorExtensionHandle>& Value) { return Value.Value == ExtensionHandle; });
}

void FPluginBrowserModule::OpenPluginEditor(TSharedRef<IPlugin> PluginToEdit, TSharedPtr<SWidget> ParentWidget, FSimpleDelegate OnEditCommitted)
{
	FPluginDescriptorEditor::OpenEditorWindow(PluginToEdit, ParentWidget, OnEditCommitted);
}

void FPluginBrowserModule::SetPluginPendingEnableState(const FString& PluginName, bool bCurrentlyEnabled, bool bPendingEnabled)
{
	if (bCurrentlyEnabled == bPendingEnabled)
	{
		PendingEnablePlugins.Remove(PluginName);
	}
	else
	{
		PendingEnablePlugins.FindOrAdd(PluginName) = bPendingEnabled;
	}
}

bool FPluginBrowserModule::GetPluginPendingEnableState(const FString& PluginName) const
{
	check(PendingEnablePlugins.Contains(PluginName));

	return PendingEnablePlugins[PluginName];
}

bool FPluginBrowserModule::HasPluginPendingEnable(const FString& PluginName) const
{
	return PendingEnablePlugins.Contains(PluginName);
}

bool FPluginBrowserModule::IsNewlyInstalledPlugin(const FString& PluginName) const
{
	return NewlyInstalledPlugins.Contains(PluginName);
}

bool FPluginBrowserModule::ShowPendingRestart() const
{
	LastQueriedExternalSources.Reset();
	IPluginManager::Get().GetExternalPluginSources(LastQueriedExternalSources);

	// For sets of equal size, inclusion denotes equality.
	const bool bExternalSourcesChanged =
		(LastQueriedExternalSources.Num() != OriginalExternalSources.Num())
		|| !OriginalExternalSources.Includes(LastQueriedExternalSources);

	return HasPluginsPendingEnable() || bExternalSourcesChanged;
}

TSharedRef<SDockTab> FPluginBrowserModule::HandleSpawnPluginBrowserTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = 
		SNew( SDockTab )
		.TabRole( ETabRole::MajorTab );

	MajorTab->SetContent(
		SNew( SPluginBrowser )
		.OnRestartClicked_Lambda([this]() -> FReply
		{
			if (OnRestartClickedDelegate.IsBound())
			{
				OnRestartClickedDelegate.Execute();
			}
			else
			{
				const bool bWarn_false = false;
				FUnrealEdMisc::Get().RestartEditor(bWarn_false);
			}
			return FReply::Handled();
		})
	);

	UpdatePreviousInstalledPlugins();

	return MajorTab;
}

TSharedRef<SDockTab> FPluginBrowserModule::HandleSpawnPluginCreatorTab(const FSpawnTabArgs& SpawnTabArgs)
{
	// Spawns the plugin creator tab with the default definition
	return SpawnPluginCreatorTab(SpawnTabArgs, nullptr);
}

TSharedRef<SDockTab> FPluginBrowserModule::SpawnPluginCreatorTab(const FSpawnTabArgs& SpawnTabArgs, TSharedPtr<IPluginWizardDefinition> PluginWizardDefinition)
{
	TSharedRef<SDockTab> ResultTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	TSharedRef<SWidget> TabContentWidget = SNew(SNewPluginWizard, ResultTab, PluginWizardDefinition);
	ResultTab->SetContent(TabContentWidget);

	return ResultTab;
}

TSharedRef<SDockTab> FPluginBrowserModule::HandleSpawnExternalDirectoriesTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	TSharedRef<SWidget> TabContent = SNew(SPluginPaths);
	MajorTab->SetContent(TabContent);

	return MajorTab;
}

void FPluginBrowserModule::OnMainFrameLoaded(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog)
{
	// Show a popup notification that allows the user to enable any new plugins
	if(!bIsRunningStartupDialog 
		&& NewlyInstalledPlugins.Num() > 0
		&& FGlobalTabmanager::Get()->GetTabPermissionList()->PassesFilter(PluginsEditorTabName))
	{
		FNotificationInfo Info(LOCTEXT("NewPluginsPopupTitle", "New plugins are available"));
		Info.bFireAndForget = true;
		Info.bUseLargeFont = true;
		Info.bUseThrobber = false;
		Info.ExpireDuration = 10.0f;
		Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NewPluginsPopupSettings", "Manage Plugins..."), LOCTEXT("NewPluginsPopupSettingsTT", "Open the plugin browser to enable plugins"), FSimpleDelegate::CreateRaw(this, &FPluginBrowserModule::OnNewPluginsPopupSettingsClicked)));
		Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NewPluginsPopupDismiss", "Dismiss"), LOCTEXT("NewPluginsPopupDismissTT", "Dismiss this notification"), FSimpleDelegate::CreateRaw(this, &FPluginBrowserModule::OnNewPluginsPopupDismissClicked)));

		NewPluginsNotification = FSlateNotificationManager::Get().AddNotification(Info);
		NewPluginsNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FPluginBrowserModule::OnNewPluginsPopupSettingsClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(PluginsEditorTabName);
	NewPluginsNotification.Pin()->SetCompletionState(SNotificationItem::CS_None);
	NewPluginsNotification.Pin()->SetExpireDuration(0.0f);
	NewPluginsNotification.Pin()->SetFadeOutDuration(0.0f);
	NewPluginsNotification.Pin()->ExpireAndFadeout();
}

void FPluginBrowserModule::OnNewPluginsPopupDismissClicked()
{
	NewPluginsNotification.Pin()->SetCompletionState(SNotificationItem::CS_None);
	NewPluginsNotification.Pin()->SetExpireDuration(0.0f);
	NewPluginsNotification.Pin()->SetFadeOutDuration(0.0f);
	NewPluginsNotification.Pin()->ExpireAndFadeout();
	UpdatePreviousInstalledPlugins();
}

void FPluginBrowserModule::UpdatePreviousInstalledPlugins()
{
	GConfig->SetArray(TEXT("PluginBrowser"), TEXT("InstalledPlugins"), InstalledPlugins, GEditorPerProjectIni);
}

#undef LOCTEXT_NAMESPACE
