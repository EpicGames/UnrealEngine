// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "IPluginBrowser.h"
#include "Interfaces/IPluginManager.h"

class IModuleInterface;
class IPluginWizardDefinition;
class SDockTab;
class SNotificationItem;
class SWindow;

class FSpawnTabArgs;

DECLARE_MULTICAST_DELEGATE(FOnNewPluginCreated);

DECLARE_LOG_CATEGORY_EXTERN(LogPluginBrowser, Log, All);

class FPluginBrowserModule : public IPluginBrowser
{
public:
	/** Accessor for the module interface */
	static FPluginBrowserModule& Get()
	{
		return FModuleManager::Get().GetModuleChecked<FPluginBrowserModule>(TEXT("PluginBrowser"));
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** IPluginsEditorFeature implementation */
	virtual void RegisterPluginTemplate(TSharedRef<FPluginTemplateDescription> Template) override;
	virtual void UnregisterPluginTemplate(TSharedRef<FPluginTemplateDescription> Template) override;
	virtual FPluginEditorExtensionHandle RegisterPluginEditorExtension(FOnPluginBeingEdited Extension) override;
	virtual void UnregisterPluginEditorExtension(FPluginEditorExtensionHandle ExtensionHandle) override;
	virtual const TArray<TSharedRef<FPluginTemplateDescription>>& GetDefaultPluginTemplates() const override { return DefaultPluginTemplates; }
	virtual void OpenPluginEditor(TSharedRef<IPlugin> PluginToEdit, TSharedPtr<SWidget> ParentWidget, FSimpleDelegate OnEditCommitted) override;

	/** Gets a delegate so that you can register/unregister to receive callbacks when plugins are created */
	FOnNewPluginCreated& OnNewPluginCreated() {return NewPluginCreatedDelegate;}

	/** Broadcasts callback to notify registrants that a plugin has been created */
	void BroadcastNewPluginCreated() const {NewPluginCreatedDelegate.Broadcast();}

	virtual FOnLaunchReferenceViewer& OnLaunchReferenceViewerDelegate() override { return LaunchReferenceViewerDelegate; }
	virtual FOnPluginDirectoriesChanged& OnPluginDirectoriesChanged() override { return OnPluginDirectoriesChangedDelegate; }
	virtual FSimpleDelegate& OnRestartClicked() override { return OnRestartClickedDelegate; }

	/**
	 * Sets whether a plugin is pending enable/disable
	 * @param PluginName The name of the plugin
	 * @param bCurrentlyEnabled The current state of this plugin, so that we can decide whether a change is no longer pending
	 * @param bPendingEnabled Whether to set this plugin to pending enable or disable
	 */
	void SetPluginPendingEnableState(const FString& PluginName, bool bCurrentlyEnabled, bool bPendingEnabled);

	/**
	 * Gets whether a plugin is pending enable/disable
	 * This should only be used when you know this is the case after using HasPluginPendingEnable
	 * @param PluginName The name of the plugin
	 */
	bool GetPluginPendingEnableState(const FString& PluginName) const;

	/** Whether there are any plugins pending enable/disable */
	bool HasPluginsPendingEnable() const {return PendingEnablePlugins.Num() > 0;}

	/**
	 * Whether a specific plugin is pending enable/disable
	 * @param PluginName The name of the plugin
	 */
	bool HasPluginPendingEnable(const FString& PluginName) const;

	/** Checks whether the given plugin should be displayed with a 'NEW' label */
	bool IsNewlyInstalledPlugin(const FString& PluginName) const;

	/** Whether the restart editor notice should be displayed. */
	bool ShowPendingRestart() const;

	/** ID name for the plugins editor major tab */
	static const FName PluginsEditorTabName;

	/** ID name for the plugin creator tab */
	static const FName PluginCreatorTabName;

	/** ID name for the external plugin directories tab */
	static const FName ExternalDirectoriesTabName;

	/** Spawns the plugin creator tab with a specific wizard definition */
	virtual TSharedRef<SDockTab> SpawnPluginCreatorTab(const FSpawnTabArgs& SpawnTabArgs, TSharedPtr<IPluginWizardDefinition> PluginWizardDefinition) override;

	virtual const TArray<TSharedRef<FPluginTemplateDescription>>& GetAddedPluginTemplates() const override { return AddedPluginTemplates; }

	const TArray<TPair<FOnPluginBeingEdited, FPluginEditorExtensionHandle>>& GetCustomizePluginEditingDelegates() { return CustomizePluginEditingDelegates; }

private:
	/** Called to spawn the plugin browser tab */
	TSharedRef<SDockTab> HandleSpawnPluginBrowserTab(const FSpawnTabArgs& SpawnTabArgs);

	/** Called to spawn the plugin creator tab */
	TSharedRef<SDockTab> HandleSpawnPluginCreatorTab(const FSpawnTabArgs& SpawnTabArgs);

	/** Called to spawn the external directories tab */
	TSharedRef<SDockTab> HandleSpawnExternalDirectoriesTab(const FSpawnTabArgs& SpawnTabArgs);

	/** Callback for the main frame finishing load */
	void OnMainFrameLoaded(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog);

	/** Callback for when the user selects to edit installed plugins */
	void OnNewPluginsPopupSettingsClicked();

	/** Callback for when the user selects to edit installed plugins */
	void OnNewPluginsPopupDismissClicked();

	/** Updates the user's config file with the list of installed plugins that they've seen. */
	void UpdatePreviousInstalledPlugins();

	/** Register menu extensions for the content browser */
	void AddContentBrowserMenuExtensions();

	/** Generate Default Plugin Template Descriptions */
	void GenerateDefaultPluginTemplateDescriptions();

	/** Delegate to call when the restart button in the pending restart notice is clicked. */
	FSimpleDelegate OnRestartClickedDelegate;

	/** List of default plugin templates */
	TArray<TSharedRef<FPluginTemplateDescription>> DefaultPluginTemplates;

	/** List of added plugin templates */
	TArray<TSharedRef<FPluginTemplateDescription>> AddedPluginTemplates;

	/** Additional customizers of plugin editing */
	TArray<TPair<FOnPluginBeingEdited, FPluginEditorExtensionHandle>> CustomizePluginEditingDelegates;
	int32 EditorExtensionCounter = 0;

	/** List of plugins that are pending enable/disable */
	TMap<FString, bool> PendingEnablePlugins;

	/** List of all the installed plugins (as opposed to built-in engine plugins) */
	TArray<FString> InstalledPlugins;

	/** List of plugins that have been recently installed */
	TSet<FString> NewlyInstalledPlugins;

	/** External plugin sources configuration as captured at startup. */
	TSet<FExternalPluginPath> OriginalExternalSources;

	/** Most recently queried plugin sources configuration. */
	mutable TSet<FExternalPluginPath> LastQueriedExternalSources;

	/** Delegate called when a new plugin is created */
	FOnNewPluginCreated NewPluginCreatedDelegate;

	/** Delegate that if bound the Plugin Browser will show the Reference Viewer button. Delegate called when the button is clicked */
	FOnLaunchReferenceViewer LaunchReferenceViewerDelegate;

	/** Called when the external plugin directories configuration is modified via the browser. */
	FOnPluginDirectoriesChanged OnPluginDirectoriesChangedDelegate;

	/** Notification popup that new plugins are available */
	TWeakPtr<SNotificationItem> NewPluginsNotification;
};
