// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



template <typename ItemType> class TTextFilter;

class IPlugin;

typedef TTextFilter< const IPlugin* > FPluginTextFilter;
