// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginDescriptorEditor.h"
#include "DetailsViewArgs.h"
#include "Misc/MessageDialog.h"
#include "IDetailsView.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "LocalizationDescriptor.h"
#include "Widgets/Text/STextBlock.h"
#include "PluginReferenceDescriptor.h"
#include "Widgets/Input/SButton.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Interfaces/IPluginManager.h"
#include "PluginStyle.h"
#include "PluginMetadataObject.h"
#include "PropertyEditorModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "PluginListTile"

void FPluginDescriptorEditor::OpenEditorWindow(TSharedRef<IPlugin> Plugin, TSharedPtr<SWidget> ParentWidget, FSimpleDelegate OnEditCommitted)
{
	// Construct the plugin metadata object using the descriptor for this plugin
	UPluginMetadataObject* MetadataObject = NewObject<UPluginMetadataObject>();
	MetadataObject->TargetIconPath = Plugin->GetBaseDir() / TEXT("Resources/Icon128.png");
	MetadataObject->PopulateFromPlugin(Plugin);
	MetadataObject->AddToRoot();

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;

	TSharedRef<IDetailsView> PropertyView = EditModule.CreateDetailView(DetailsViewArgs);
	PropertyView->SetObject(MetadataObject, true);

	// Create the window
	TSharedRef<SWindow> PropertiesWindow = SNew(SWindow)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(700.0f, 700.0f))
		.Title(LOCTEXT("PluginMetadata", "Plugin Properties"));

	PropertiesWindow->SetContent(
			SNew(SBorder)
			.Padding(FMargin(8.0f, 8.0f))
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(5.0f, 10.0f, 5.0f, 5.0f))
				[
					SNew(STextBlock)
					.Font(FPluginStyle::Get()->GetFontStyle(TEXT("PluginMetadataNameFont")))
					.Text(FText::FromString(Plugin->GetName()))
				]

				+ SVerticalBox::Slot()
				.Padding(5)
				[
					PropertyView
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.ContentPadding(FMargin(20.0f, 2.0f))
					.Text(LOCTEXT("OkButtonLabel", "Ok"))
					.OnClicked(FOnClicked::CreateStatic(&FPluginDescriptorEditor::OnEditPluginFinished, MetadataObject, TSharedPtr<IPlugin>(Plugin), OnEditCommitted, TWeakPtr<SWindow>(PropertiesWindow)))
				]
			]
		);

	FSlateApplication::Get().AddModalWindow(PropertiesWindow, ParentWidget);
}

FReply FPluginDescriptorEditor::OnEditPluginFinished(UPluginMetadataObject* MetadataObject, TSharedPtr<IPlugin> Plugin, FSimpleDelegate OnEditCommitted, TWeakPtr<SWindow> WeakWindow)
{
	FPluginDescriptor OldDescriptor = Plugin->GetDescriptor();

	bool bAutoSortPluginDependencies = false;
	if (GConfig
		&& GConfig->GetBool(TEXT("PluginDescriptorSettings"), TEXT("bAutoSortPluginDependencies"), bAutoSortPluginDependencies, GEditorIni)
		&& bAutoSortPluginDependencies)
	{
		MetadataObject->SortDependencies();
	}

	// Update the descriptor with the new metadata
	FPluginDescriptor NewDescriptor = OldDescriptor;
	MetadataObject->CopyIntoDescriptor(NewDescriptor);
	MetadataObject->RemoveFromRoot();

	// Close the properties window
	TSharedPtr<SWindow> PropertiesWindow = WeakWindow.Pin();
	if (PropertiesWindow.IsValid())
	{
		PropertiesWindow->RequestDestroyWindow();
	}

	// Write both to strings
	FString OldText;
	OldDescriptor.Write(OldText);
	FString NewText;
	NewDescriptor.Write(NewText);
	if(OldText.Compare(NewText, ESearchCase::CaseSensitive) != 0)
	{
		FString DescriptorFileName = Plugin->GetDescriptorFileName();

		// First attempt to check out the file if SCC is enabled
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		if(SourceControlModule.IsEnabled())
		{
			FScopedSlowTask SlowTask(0, LOCTEXT("CheckOutPlugin", "Checking out plugin..."));

			ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();
			TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = SourceControlProvider.GetState(DescriptorFileName, EStateCacheUsage::ForceUpdate);
			if(SourceControlState.IsValid() && SourceControlState->CanCheckout())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), DescriptorFileName);
			}
		}

		// Write to the file and update the in-memory metadata
		FText FailReason;
		if(!Plugin->UpdateDescriptor(NewDescriptor, FailReason))
		{
			FMessageDialog::Open(EAppMsgType::Ok, FailReason);
		}
	}

	OnEditCommitted.ExecuteIfBound();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
