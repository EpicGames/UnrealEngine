// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FReply;

class UPluginMetadataObject;
class IPlugin;
class SWindow;
class SWidget;

/**
 * Creates a window to edit a .uplugin file
 */
class FPluginDescriptorEditor
{
public:
	static void OpenEditorWindow(TSharedRef<IPlugin> PluginToEdit, TSharedPtr<SWidget> ParentWidget, FSimpleDelegate OnEditCommitted);

private:
	static FReply OnEditPluginFinished(UPluginMetadataObject* MetadataObject, TSharedPtr<IPlugin> Plugin, FSimpleDelegate OnEditCommitted, TWeakPtr<SWindow> WeakWindow);
};

