// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginMetadataObject.h"
#include "Misc/Paths.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Interfaces/IPluginManager.h"
#include "PluginDisallowedDescriptor.h"
#include "PluginReferenceDescriptor.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "SExternalImageReference.h"
#include "PluginBrowserModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PluginMetadataObject)

void FPluginReferenceMetadata::PopulateFromDescriptor(const FPluginReferenceDescriptor& InDescriptor)
{
	Name = InDescriptor.Name;
	bEnabled = InDescriptor.bEnabled;
	bOptional = InDescriptor.bOptional;
	bActivate = InDescriptor.bActivate;
}

void FPluginReferenceMetadata::CopyIntoDescriptor(FPluginReferenceDescriptor& OutDescriptor) const
{
	OutDescriptor.Name = Name;
	OutDescriptor.bEnabled = bEnabled;
	OutDescriptor.bOptional = bOptional;
	OutDescriptor.bActivate = bActivate;
}

void FPluginDisallowedMetadata::PopulateFromDescriptor(const FPluginDisallowedDescriptor& InDescriptor)
{
	Name = InDescriptor.Name;
	Comment = InDescriptor.Comment;
}

void FPluginDisallowedMetadata::CopyIntoDescriptor(FPluginDisallowedDescriptor& OutDescriptor) const
{
	OutDescriptor.Name = Name;
	OutDescriptor.Comment = Comment;
}

UPluginMetadataObject::UPluginMetadataObject(const FObjectInitializer& ObjectInitializer)
{
}

void UPluginMetadataObject::SortDependencies()
{
	Plugins.StableSort([] (const FPluginReferenceMetadata& First, const FPluginReferenceMetadata& Second)
		{
			return First.Name < Second.Name;
		});
	DisallowedPlugins.StableSort([] (const FPluginDisallowedMetadata& First, const FPluginDisallowedMetadata& Second)
		{
			return First.Name < Second.Name;
		});
}

void UPluginMetadataObject::PopulateFromPlugin(TSharedPtr<IPlugin> InPlugin)
{
	SourcePlugin = InPlugin;

	const FPluginDescriptor& InDescriptor = InPlugin->GetDescriptor();
	Version = InDescriptor.Version;
	VersionName = InDescriptor.VersionName;
	FriendlyName = InDescriptor.FriendlyName;
	Description = InDescriptor.Description;
	Category = InDescriptor.Category;
	CreatedBy = InDescriptor.CreatedBy;
	CreatedByURL = InDescriptor.CreatedByURL;
	DocsURL = InDescriptor.DocsURL;
	MarketplaceURL = InDescriptor.MarketplaceURL;
	SupportURL = InDescriptor.SupportURL;
	EditorCustomVirtualPath = InDescriptor.EditorCustomVirtualPath;
	bCanContainContent = InDescriptor.bCanContainContent;
	bIsBetaVersion = InDescriptor.bIsBetaVersion;
	bIsEnabledByDefault = (InDescriptor.EnabledByDefault == EPluginEnabledByDefault::Enabled);
	bExplicitlyLoaded = InDescriptor.bExplicitlyLoaded;
	bIsSealed = InDescriptor.bIsSealed;
	bNoCode = InDescriptor.bNoCode;

	Plugins.Reset(InDescriptor.Plugins.Num());
	for (const FPluginReferenceDescriptor& PluginRefDesc : InDescriptor.Plugins)
	{
		FPluginReferenceMetadata& PluginRef = Plugins.AddDefaulted_GetRef();
		PluginRef.PopulateFromDescriptor(PluginRefDesc);
	}

	DisallowedPlugins.Reset(InDescriptor.DisallowedPlugins.Num());
	for (const FPluginDisallowedDescriptor& PluginRefDesc : InDescriptor.DisallowedPlugins)
	{
		FPluginDisallowedMetadata& PluginRef = DisallowedPlugins.AddDefaulted_GetRef();
		PluginRef.PopulateFromDescriptor(PluginRefDesc);
	}
}

void UPluginMetadataObject::CopyIntoDescriptor(FPluginDescriptor& OutDescriptor) const
{
	OutDescriptor.Version = Version;
	OutDescriptor.VersionName = VersionName;
	OutDescriptor.FriendlyName = FriendlyName;
	OutDescriptor.Description = Description;
	OutDescriptor.Category = Category;
	OutDescriptor.CreatedBy = CreatedBy;
	OutDescriptor.CreatedByURL = CreatedByURL;
	OutDescriptor.DocsURL = DocsURL;
	OutDescriptor.MarketplaceURL = MarketplaceURL;
	OutDescriptor.EditorCustomVirtualPath = EditorCustomVirtualPath;
	OutDescriptor.SupportURL = SupportURL;
	OutDescriptor.bCanContainContent = bCanContainContent;
	OutDescriptor.bIsBetaVersion = bIsBetaVersion;
	OutDescriptor.bIsSealed = bIsSealed;
	OutDescriptor.bNoCode = bNoCode;

	TArray<FPluginReferenceDescriptor> NewPlugins;
	NewPlugins.Reserve(Plugins.Num());

	for (const FPluginReferenceMetadata& PluginRefMetadata : Plugins)
	{
		if (PluginRefMetadata.Name.IsEmpty())
		{
			continue;
		}

		FPluginReferenceDescriptor& NewPluginRefDesc = NewPlugins.AddDefaulted_GetRef();

		if (FPluginReferenceDescriptor* OldPluginRefDesc = OutDescriptor.Plugins.FindByPredicate([&PluginRefMetadata](const FPluginReferenceDescriptor& Item) { return Item.Name == PluginRefMetadata.Name; }))
		{
			NewPluginRefDesc = *OldPluginRefDesc;
			OldPluginRefDesc->Name.Empty(); // Clear its name so we don't find it again (multiple entries with same name would be wrong but still have to handle it somehow)
		}

		PluginRefMetadata.CopyIntoDescriptor(NewPluginRefDesc);
	}

	OutDescriptor.Plugins = MoveTemp(NewPlugins);

	{
		TArray<FPluginDisallowedDescriptor> NewDisallowedPlugins;
		NewDisallowedPlugins.Reserve(DisallowedPlugins.Num());

		for (const FPluginDisallowedMetadata& DisallowedRefMetadata : DisallowedPlugins)
		{
			if (DisallowedRefMetadata.Name.IsEmpty())
			{
				continue;
			}

			FPluginDisallowedDescriptor& NewDisallowedRefDesc = NewDisallowedPlugins.AddDefaulted_GetRef();
			DisallowedRefMetadata.CopyIntoDescriptor(NewDisallowedRefDesc);
		}

		OutDescriptor.DisallowedPlugins = MoveTemp(NewDisallowedPlugins);
	}

	// Apply any edits done by an extension
	for (const TSharedPtr<FPluginEditorExtension>& Extension : Extensions)
	{
		Extension->CommitEdits(OutDescriptor);
	}
}

TArray<FString> UPluginMetadataObject::GetAvailablePluginDependencies() const
{
	TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();
	TArray<FString> Result;
	Result.Reserve(AllPlugins.Num());

	const FString MyName = SourcePlugin.Pin()->GetName();

	// This lists all plugins that don't directly depend on the plugin being edited (so a multi-hop chain would still cause a problem)
	for (const TSharedRef<IPlugin>& Plugin : AllPlugins)
	{
		if (Plugin == SourcePlugin)
		{
			continue;
		}

		// Can't depend on plugins that are marked as sealed.
		if (Plugin->GetDescriptor().bIsSealed)
		{
			continue;
		}

		bool bDependencyDisallowed = false;
		for (const FPluginDisallowedDescriptor& DisallowedPlugin : Plugin->GetDescriptor().DisallowedPlugins)
		{
			if (Plugin->GetName() == DisallowedPlugin.Name)
			{
				bDependencyDisallowed = true;
				break;
			}
		}

		bool bDependsOnMe = false;
		for (const FPluginReferenceDescriptor& Dependency : Plugin->GetDescriptor().Plugins)
		{
			if (Dependency.Name == MyName)
			{
				bDependsOnMe = true;
				break;
			}
		}

		if (!bDependsOnMe && !bDependencyDisallowed)
		{
			Result.Add(Plugin->GetName());
		}
	}

	return Result;
}

TArray<FString> UPluginMetadataObject::GetDisallowedPluginsOptions() const
{
	TArray<TSharedRef<IPlugin>> AllPlugins = IPluginManager::Get().GetDiscoveredPlugins();
	TArray<FString> Result;
	Result.Reserve(AllPlugins.Num());

	for (const TSharedRef<IPlugin>& Plugin : AllPlugins)
	{
		if (Plugin == SourcePlugin)
		{
			continue;
		}

		Result.Add(Plugin->GetName());
	}

	return Result;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FPluginMetadataCustomization::MakeInstance()
{
	return MakeShareable(new FPluginMetadataCustomization());
}

void FPluginMetadataCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	if(Objects.Num() == 1 && Objects[0].IsValid())
	{
		UPluginMetadataObject* PluginMetadata = Cast<UPluginMetadataObject>(Objects[0].Get());
		check(PluginMetadata);

		// Run any external customizations
		PluginMetadata->Extensions.Reset();
		FPluginEditingContext PluginEditingContext;
		PluginEditingContext.PluginBeingEdited = PluginMetadata->SourcePlugin.Pin();
		for (const auto& KVP : FPluginBrowserModule::Get().GetCustomizePluginEditingDelegates())
		{
			TSharedPtr<FPluginEditorExtension> Extension = KVP.Key.Execute(PluginEditingContext, DetailBuilder);
			if (Extension.IsValid())
			{
				PluginMetadata->Extensions.Add(Extension);
			}
		}
		
		if(PluginMetadata->TargetIconPath.Len() > 0)
		{
			// Get the current icon path
			FString CurrentIconPath = PluginMetadata->TargetIconPath;
			if(!FPaths::FileExists(CurrentIconPath))
			{
				CurrentIconPath = IPluginManager::Get().FindPlugin(TEXT("PluginBrowser"))->GetBaseDir() / TEXT("Resources") / TEXT("DefaultIcon128.png");
			}

			// Add the customization to edit the icon row
			IDetailCategoryBuilder& ImageCategory = DetailBuilder.EditCategory(TEXT("Icon"));
			const FText IconDesc(NSLOCTEXT("PluginBrowser", "PluginIcon", "Icon"));
			ImageCategory.AddCustomRow(IconDesc)
			.NameContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding( FMargin( 0, 1, 0, 1 ) )
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Text(IconDesc)
					.Font(DetailBuilder.GetDetailFont())
				]
			]
			.ValueContent()
			.MaxDesiredWidth(500.0f)
			.MinDesiredWidth(100.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SExternalImageReference, CurrentIconPath, PluginMetadata->TargetIconPath)
					.FileDescription(IconDesc)
					.RequiredSize(FIntPoint(128, 128))
				]
			];
		}
	}
}

