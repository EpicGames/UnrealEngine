// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "PluginMetadataObject.generated.h"

class IDetailLayoutBuilder;
struct FPluginDescriptor;
struct FPluginDisallowedDescriptor;
struct FPluginReferenceDescriptor;
class IPlugin;
struct FPluginEditorExtension;

/**
 * We use this object to display plugin reference properties using details view.
 */
USTRUCT()
struct FPluginReferenceMetadata
{
public:
	GENERATED_BODY()

	/** Name of the dependency plugin */
	UPROPERTY(EditAnywhere, Category = "Plugin Reference", meta=(GetOptions=GetAvailablePluginDependencies))
	FString Name;

	/** Whether the dependency plugin is optional meaning it will be silently ignored if not present */
	UPROPERTY(EditAnywhere, Category = "Plugin Reference")
	bool bOptional = false;

	/** Whether the dependency plugin should be enabled by default */
	UPROPERTY(EditAnywhere, Category = "Plugin Reference")
	bool bEnabled = true;

	/** Whether the dependency plugin should be activated when this plugin is activated. This only works if the dependency is a GameFeaturePlugin */
	UPROPERTY(EditAnywhere, Category = "Plugin Reference")
	bool bActivate = false;

	/**
	 * Populate the fields of this object from an existing descriptor.
	 */
	void PopulateFromDescriptor(const FPluginReferenceDescriptor& InDescriptor);

	/**
	 * Copy the metadata fields into a plugin descriptor.
	 */
	void CopyIntoDescriptor(FPluginReferenceDescriptor& OutDescriptor) const;
};

USTRUCT()
struct FPluginDisallowedMetadata
{
public:
	GENERATED_BODY()

	/** Name of the dependency plugin */
	UPROPERTY(EditAnywhere, Category = Details, meta = (GetOptions = GetDisallowedPluginsOptions))
	FString Name;

	UPROPERTY(EditAnywhere, Category = Details)
	FString Comment;

	/**
	 * Populate the fields of this object from an existing descriptor.
	 */
	void PopulateFromDescriptor(const FPluginDisallowedDescriptor& InDescriptor);

	/**
	 * Copy the metadata fields into a plugin descriptor.
	 */
	void CopyIntoDescriptor(FPluginDisallowedDescriptor& OutDescriptor) const;	
};

/**
 * We use this object to display plugin properties using details view.
 */
UCLASS()
class UPluginMetadataObject : public UObject
{
public:
	GENERATED_BODY()

	/* Default constructor */
	UPluginMetadataObject(const FObjectInitializer& ObjectInitializer);

	/** Path to this this plugin's icon */
	FString TargetIconPath;

	/** Version number for the plugin.  The version number must increase with every version of the plugin, so that the system
	can determine whether one version of a plugin is newer than another, or to enforce other requirements.  This version
	number is not displayed in front-facing UI.  Use the VersionName for that. */
	UPROPERTY(VisibleAnywhere, Category = Details)
	int32 Version;

	/** Name of the version for this plugin.  This is the front-facing part of the version number.  It doesn't need to match
	the version number numerically, but should be updated when the version number is increased accordingly. */
	UPROPERTY(EditAnywhere, Category = Details)
	FString VersionName;

	/** Friendly name of the plugin */
	UPROPERTY(EditAnywhere, Category = Details)
	FString FriendlyName;

	/** Description of the plugin */
	UPROPERTY(EditAnywhere, Category = Details)
	FString Description;

	/** The category that this plugin belongs to */
	UPROPERTY(EditAnywhere, Category = Details)
	FString Category;

	/** The company or individual who created this plugin.  This is an optional field that may be displayed in the user interface. */
	UPROPERTY(EditAnywhere, Category = Details)
	FString CreatedBy;

	/** Hyperlink URL string for the company or individual who created this plugin.  This is optional. */
	UPROPERTY(EditAnywhere, Category = Details)
	FString CreatedByURL;

	/** Documentation URL string. */
	UPROPERTY(EditAnywhere, Category = Details)
	FString DocsURL;

	/** Marketplace URL string. */
	UPROPERTY(EditAnywhere, Category = Details)
	FString MarketplaceURL;

	/** Support URL/email for this plugin. Email addresses must be prefixed with 'mailto:' */
	UPROPERTY(EditAnywhere, Category = Details)
	FString SupportURL;

	/** Optional custom virtual path to display in editor to better organize. Inserted just before this plugin's directory in the path: /All/Plugins/EditorCustomVirtualPath/PluginName */
	UPROPERTY(EditAnywhere, Category = Details)
	FString EditorCustomVirtualPath;

	/** Can this plugin contain content? */
	UPROPERTY(EditAnywhere, Category = Details)
	bool bCanContainContent;

	/** Marks the plugin as beta in the UI */
	UPROPERTY(EditAnywhere, Category = Details)
	bool bIsBetaVersion;

	/** If true, the plugin will be enabled by default without needing to modify the project file. This cannot be changed while the editor is running */
	UPROPERTY(VisibleAnywhere, Category = Details)
	bool bIsEnabledByDefault;

	/** If true, the plugin will not load at startup and must be explicitly loaded. This cannot be changed while the editor is running */
	UPROPERTY(VisibleAnywhere, Category = Details)
	bool bExplicitlyLoaded;

	/** If true, prevents other plugins from depending on this plugin */
	UPROPERTY(EditAnywhere, Category = Details)
	bool bIsSealed;

	UPROPERTY(EditAnywhere, Category = Details)
	bool bNoCode;

	/** Plugins used by this plugin */
	UPROPERTY(EditAnywhere, Category = Dependencies, meta=(TitleProperty=Name))
	TArray<FPluginReferenceMetadata> Plugins;

	/** Plugins that cannot be used by this plugin */
	UPROPERTY(EditAnywhere, Category = Dependencies, meta = (DisplayName = "Disallowed"))
	TArray<FPluginDisallowedMetadata> DisallowedPlugins;

	/** Plugin this proxy object was constructed from */
	TWeakPtr<IPlugin> SourcePlugin;

	/** Editing extensions */
	TArray<TSharedPtr<FPluginEditorExtension>> Extensions;

	/** Sort the plugins and disallowed plugins references in ascending name order */
	UFUNCTION(CallInEditor, Category=Dependencies, meta = (DisplayName = "Sort dependencies"))
	void SortDependencies();

	/**
	 * Populate the fields of this object from an existing descriptor.
	 */
	void PopulateFromPlugin(TSharedPtr<IPlugin> InPlugin);

	/**
	 * Copy the metadata fields into a plugin descriptor.
	 */
	void CopyIntoDescriptor(FPluginDescriptor& OutDescriptor) const;

	UFUNCTION()
	TArray<FString> GetAvailablePluginDependencies() const;

	UFUNCTION()
	TArray<FString> GetDisallowedPluginsOptions() const;
};

/**
 * Detail customization to allow editing the plugin's icon
 */
class FPluginMetadataCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};
