// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/SlateTypes.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/CoreStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/SlateStyle.h"

#define RootToContentDir FPluginStyle::InContent
#define RootToCoreContentDir StyleSet->RootToCoreContentDir

#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"

FString FPluginStyle::InContent( const FString& RelativePath, const TCHAR* Extension )
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("PluginBrowser"))->GetContentDir();
	return ( ContentDir / RelativePath ) + Extension;
}

TSharedPtr< FSlateStyleSet > FPluginStyle::StyleSet = NULL;
TSharedPtr< class ISlateStyle > FPluginStyle::Get() { return StyleSet; }

void FPluginStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon9x19(9.0f, 19.0f);
	const FVector2D Icon10x10(10.0f, 10.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon22x22(22.0f, 22.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon27x31(27.0f, 31.0f);
	const FVector2D Icon26x26(26.0f, 26.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon75x82(75.0f, 82.0f);
	const FVector2D Icon360x32(360.0f, 32.0f);
	const FVector2D Icon171x39(171.0f, 39.0f);
	const FVector2D Icon170x50(170.0f, 50.0f);
	const FVector2D Icon267x140(170.0f, 50.0f);

	// Only register once
	if( StyleSet.IsValid() )
	{
		return;
	}

	StyleSet = MakeShareable( new FSlateStyleSet("PluginStyle") );
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Plugins Manager
	const FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FTextBlockStyle ButtonText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText");
	FTextBlockStyle LargeText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Text.Large");

	StyleSet->Set( "Plugins.TabIcon", new IMAGE_BRUSH_SVG( "Plugins", Icon16x16 ) );
	StyleSet->Set( "Plugins.BreadcrumbArrow", new IMAGE_BRUSH( "SmallArrowRight", Icon10x10 ) );
	StyleSet->Set( "Plugins.ListBorder", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f));
	StyleSet->Set( "Plugins.RestartWarningBorder", new FSlateRoundedBoxBrush(FStyleColors::Panel , 5.0f, FStyleColors::Warning, 1.0f));

	FTextBlockStyle WarningText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FStyleColors::White);

	StyleSet->Set("Plugins.WarningText", WarningText);

	//Category Tree Item
	const float IconSize = 16.0f;
	const float PaddingAmount = 2.0f;

	StyleSet->Set( "CategoryTreeItem.IconSize", IconSize );
	StyleSet->Set( "CategoryTreeItem.PaddingAmount", PaddingAmount );
			
	StyleSet->Set( "CategoryTreeItem.BuiltIn", new IMAGE_BRUSH( "icon_plugins_builtin_20x", Icon20x20 ) );
	StyleSet->Set( "CategoryTreeItem.Installed", new IMAGE_BRUSH( "icon_plugins_installed_20x", Icon20x20 ) );
	StyleSet->Set( "CategoryTreeItem.LeafItemWithPlugin", new IMAGE_BRUSH( "hiererchy_16x", Icon12x12 ) );
	StyleSet->Set( "CategoryTreeItem.ExpandedCategory", new IMAGE_BRUSH( "FolderOpen", FVector2D(18, 16) ) );
	StyleSet->Set( "CategoryTreeItem.Category", new IMAGE_BRUSH( "FolderClosed", FVector2D(18, 16) ) );

	//Root Category Tree Item
	const float ExtraTopPadding = 12.f;
	const float ExtraBottomPadding = 8.f;
	const float AllPluginsExtraTopPadding = 9.f;
	const float AllPluginsExtraBottomPadding = 7.f;

	StyleSet->Set( "CategoryTreeItem.Root.BackgroundBrush", new FSlateNoResource );
	StyleSet->Set( "CategoryTreeItem.Root.BackgroundPadding", FMargin( PaddingAmount, PaddingAmount + ExtraTopPadding, PaddingAmount, PaddingAmount + ExtraBottomPadding) );
	StyleSet->Set("CategoryTreeItem.Root.AllPluginsBackgroundPadding", FMargin(PaddingAmount, PaddingAmount + AllPluginsExtraTopPadding, PaddingAmount, PaddingAmount + AllPluginsExtraBottomPadding));

	FTextBlockStyle Text = FTextBlockStyle(ButtonText);
	Text.ColorAndOpacity = FStyleColors::Foreground;
	Text.TransformPolicy = ETextTransformPolicy::ToUpper;
	StyleSet->Set( "CategoryTreeItem.Root.Text", Text );


	FTextBlockStyle RootPluginCountText = FTextBlockStyle( NormalText );
	StyleSet->Set( "CategoryTreeItem.Root.PluginCountText", RootPluginCountText);

	//Subcategory Tree Item
	StyleSet->Set( "CategoryTreeItem.BackgroundBrush", new FSlateNoResource );
	StyleSet->Set( "CategoryTreeItem.BackgroundPadding", FMargin( PaddingAmount ) );


	FTextBlockStyle CategoryText = FTextBlockStyle( NormalText );
	CategoryText.ColorAndOpacity = FStyleColors::Foreground;
	StyleSet->Set( "CategoryTreeItem.Text", CategoryText);

	FTextBlockStyle PluginCountText = FTextBlockStyle( NormalText );
	StyleSet->Set( "CategoryTreeItem.PluginCountText", PluginCountText );
	

	//Plugin Tile
	StyleSet->Set("PluginTile.RestrictedBorderImage", new FSlateRoundedBoxBrush(FStyleColors::AccentRed, 8.f));
	StyleSet->Set("PluginTile.BetaBorderImage", new FSlateRoundedBoxBrush(FStyleColors::AccentOrange.GetSpecifiedColor().CopyWithNewOpacity(0.8f), 8.f));
	StyleSet->Set("PluginTile.ExperimentalBorderImage", new FSlateRoundedBoxBrush(FStyleColors::AccentPurple, 8.f));
	StyleSet->Set("PluginTile.NewLabelBorderImage", new FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor().CopyWithNewOpacity(0.8f), 8.f));
	StyleSet->Set("PluginTile.BorderImage", new FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f));
	StyleSet->Set("PluginTile.ThumbnailBorderImage", new FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f));

	StyleSet->Set( "PluginTile.Padding", PaddingAmount );

	const float HorizontalTilePadding = 8.0f;
	StyleSet->Set("PluginTile.HorizontalTilePadding", HorizontalTilePadding);

	const float VerticalTilePadding = 4.0f;
	StyleSet->Set("PluginTile.VerticalTilePadding", VerticalTilePadding);

	const float ThumbnailImageSize = 69.0f;
	StyleSet->Set( "PluginTile.ThumbnailImageSize", ThumbnailImageSize );

	FString ThumbnailImageFallBack = IPluginManager::Get().FindPlugin(TEXT("PluginBrowser"))->GetBaseDir() / TEXT("Resources/DefaultIcon128.png");
	StyleSet->Set( "PluginTile.ThumbnailImageFallBack", new FSlateImageBrush(ThumbnailImageFallBack, CoreStyleConstants::Icon128x128) );

	StyleSet->Set( "PluginTile.BackgroundBrush", new FSlateNoResource );
	StyleSet->Set( "PluginTile.BackgroundPadding", FMargin( PaddingAmount ) );

	FTextBlockStyle NameText = FTextBlockStyle( LargeText )
		.SetColorAndOpacity( FStyleColors::White );
	StyleSet->Set( "PluginTile.NameText", NameText );

	FTextBlockStyle DescriptionText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity( FStyleColors::Foreground );
	StyleSet->Set("PluginTile.DescriptionText", DescriptionText);


	FTextBlockStyle BetaText = FTextBlockStyle( NormalText )
		.SetColorAndOpacity( FStyleColors::White );
	StyleSet->Set( "PluginTile.BetaText", BetaText );


	FTextBlockStyle VersionNumberText = FTextBlockStyle(LargeText)
		.SetColorAndOpacity(FStyleColors::Foreground);
	StyleSet->Set( "PluginTile.VersionNumberText", VersionNumberText );

	FTextBlockStyle NewLabelText = FTextBlockStyle( NormalText )
		.SetColorAndOpacity( FLinearColor( 0.05f, 0.05f, 0.05f ) );
	StyleSet->Set( "PluginTile.NewLabelText", NewLabelText );

	StyleSet->Set( "PluginTile.BetaWarning", new IMAGE_BRUSH( "icon_plugins_betawarn_14px", FVector2D(14, 14) ) );

	// Metadata editor
	StyleSet->Set("PluginMetadataNameFont", DEFAULT_FONT("Bold", 18));

	// Plugin Creator
	const FButtonStyle& BaseButtonStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FButtonStyle>("Button");
	StyleSet->Set("PluginPath.BrowseButton",
		FButtonStyle(BaseButtonStyle)
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f, FStyleColors::Secondary, 2.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Hover, 2.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Header, 2.0f))
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1)));

	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
};

void FPluginStyle::Shutdown()
{
	if( StyleSet.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}

#undef RootToContentDir
