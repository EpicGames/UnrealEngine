// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FPluginStyle
{
public:

	static void Initialize();
	static void Shutdown();

	static TSharedPtr< class ISlateStyle > Get();

private:

	static FString InContent( const FString& RelativePath, const TCHAR* Extension );

private:

	static TSharedPtr< class FSlateStyleSet > StyleSet;
};
