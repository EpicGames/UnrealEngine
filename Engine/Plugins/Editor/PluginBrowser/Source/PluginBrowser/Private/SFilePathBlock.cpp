// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilePathBlock.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "PluginStyle.h"

#define LOCTEXT_NAMESPACE "FilePathBlock"

void SFilePathBlock::Construct(const FArguments& InArgs)
{
	FText BrowseForFolderToolTipText;

	const bool bReadOnlyFolderPath = InArgs._ReadOnlyFolderPath;

	if (bReadOnlyFolderPath)
	{
		BrowseForFolderToolTipText = LOCTEXT("BrowseForFolderDisabled", "You cannot modify this location");
	}
	else
	{
		BrowseForFolderToolTipText = LOCTEXT("BrowseForFolder", "Browse for a folder");
	}

	const float DesiredWidth = 700.0f;
	const float DesiredRatioForPath = 2.0f;

	const float DerivedMinWidthForPath = DesiredWidth * DesiredRatioForPath / (DesiredRatioForPath + 1.0f);
	const float DerivedMinWidthForName = DesiredWidth - DerivedMinWidthForPath;

	ChildSlot
	[
		SNew(SHorizontalBox)
		// Folder input
		+SHorizontalBox::Slot()
		.FillWidth(DesiredRatioForPath)
		[
			SNew(SBox)
			.MinDesiredWidth(DerivedMinWidthForPath)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SAssignNew(FolderPathTextBox, SEditableTextBox)
					.Text(InArgs._FolderPath)
					// Large right hand padding to make room for the browse button
					.Padding(FMargin(5.f, 3.f, 25.f, 3.f))
					.OnTextChanged(InArgs._OnFolderChanged)
					.OnTextCommitted(InArgs._OnFolderCommitted)
					.IsReadOnly(bReadOnlyFolderPath)
					.HintText(LOCTEXT("Folder", "Folder"))
					.ForegroundColor(FSlateColor::UseForeground())
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.Padding(1)
				[
					SNew(SButton)
					.ButtonStyle(FPluginStyle::Get(), "PluginPath.BrowseButton")
					.OnClicked(InArgs._OnBrowseForFolder)
					.ContentPadding(0)
					.ToolTipText(BrowseForFolderToolTipText)
					.Text(LOCTEXT("...", "..."))
					.IsEnabled(!bReadOnlyFolderPath)
				]
			]
		]
		// Name input
		+ SHorizontalBox::Slot()
		.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.MinDesiredWidth(DerivedMinWidthForName)
			[
				SAssignNew(NameTextBox, SEditableTextBox)
				.Text(InArgs._Name)
				.Padding(FMargin(5.f, 3.f))
				.HintText(InArgs._NameHint)
				.OnTextChanged(InArgs._OnNameChanged)
				.OnTextCommitted(InArgs._OnNameCommitted)
			]
		]
	];
}

void SFilePathBlock::SetFolderPathError(const FText& ErrorText)
{
	if (FolderPathTextBox.IsValid())
	{
		FolderPathTextBox->SetError(ErrorText);
	}
}

void SFilePathBlock::SetNameError(const FText& ErrorText)
{
	if (NameTextBox.IsValid())
	{
		NameTextBox->SetError(ErrorText);
	}
}

#undef LOCTEXT_NAMESPACE
