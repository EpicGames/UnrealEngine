// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"

class SEditableTextBox;

/**
 * Simple widget used to display a folder path, and a name of a file:
 * __________________________  ____________________
 * | C:\Users\Joe.Bloggs    |  | SomeFile.txt     |
 * |-------- Folder --------|  |------ Name ------|
 */
class SFilePathBlock : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SFilePathBlock )
		: _ReadOnlyFolderPath(false)
	{}
		/** Attribute specifying the text to display in the folder input */
		SLATE_ATTRIBUTE(FText, FolderPath)

		/** Attribute specifying the text to display in the name input */
		SLATE_ATTRIBUTE(FText, Name)

		/** Hint name that appears when there is no text in the name box */
		SLATE_ATTRIBUTE(FText, NameHint)

		/** If true, the folder path cannot be modified by the user */
		SLATE_ARGUMENT(bool, ReadOnlyFolderPath)

		/** Event that is triggered when the browser for folder button is clicked */
		SLATE_EVENT(FOnClicked, OnBrowseForFolder)

		/** Events for when the name field is manipulated */
		SLATE_EVENT(FOnTextChanged, OnNameChanged)
		SLATE_EVENT(FOnTextCommitted, OnNameCommitted)
		
		/** Events for when the folder field is manipulated */
		SLATE_EVENT(FOnTextChanged, OnFolderChanged)
		SLATE_EVENT(FOnTextCommitted, OnFolderCommitted)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	void SetFolderPathError(const FText& ErrorText);
	void SetNameError(const FText& ErrorText);

private:
	TSharedPtr<SEditableTextBox> FolderPathTextBox;
	TSharedPtr<SEditableTextBox> NameTextBox;
};
