// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNewPluginWizard.h"
#include "AssetToolsModule.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Misc/Paths.h"
#include "DetailsViewArgs.h"
#include "Misc/MessageDialog.h"
#include "Framework/Views/TableViewMetadata.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "IDesktopPlatform.h"
#include "Misc/App.h"
#include "Framework/Application/SlateApplication.h"
#include "LocalizationDescriptor.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "NewPluginDescriptorData.h"
#include "Widgets/Views/SListView.h"
#include "PluginReferenceDescriptor.h"
#include "PluginStyle.h"
#include "DesktopPlatformModule.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "GameProjectGenerationModule.h"
#include "PluginBrowserModule.h"
#include "SFilePathBlock.h"
#include "Interfaces/IProjectManager.h"
#include "ProjectDescriptor.h"
#include "PluginUtils.h"
#include "Interfaces/IMainFrameModule.h"
#include "GameProjectGenerationModule.h"
#include "DefaultPluginWizardDefinition.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "NewPluginPythonOptionsData.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SourceCodeNavigation.h"
#include "SPrimaryButton.h"

DEFINE_LOG_CATEGORY(LogPluginWizard);

#define LOCTEXT_NAMESPACE "NewPluginWizard"

static bool IsContentOnlyProject()
{
	const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
	return CurrentProject == nullptr || CurrentProject->Modules.Num() == 0 || !FGameProjectGenerationModule::Get().ProjectHasCodeFiles();
}

SNewPluginWizard::SNewPluginWizard()
	: bIsPluginPathValid(false)
	, bIsPluginNameValid(false)
	, bIsSelectedPathInEngine(false)
	, bIsWaitingForAssetDiscoveryToFinish(false)
{
	AbsoluteGamePluginPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::ProjectPluginsDir());
	FPaths::MakePlatformFilename(AbsoluteGamePluginPath);
	AbsoluteEnginePluginPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::EnginePluginsDir());
	FPaths::MakePlatformFilename(AbsoluteEnginePluginPath);
}

void SNewPluginWizard::Construct(const FArguments& Args, TSharedPtr<SDockTab> InOwnerTab, TSharedPtr<IPluginWizardDefinition> InPluginWizardDefinition)
{
	OwnerTab = InOwnerTab;

	PluginWizardDefinition = InPluginWizardDefinition;

	// Prepare to create the descriptor data field
	DescriptorData = NewObject<UNewPluginDescriptorData>();
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
		DetailsViewArgs.bAllowFavoriteSystem = false;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.bHideSelectionTip = true;
	}
	TSharedPtr<IDetailsView> DescriptorDetailView = EditModule.CreateDetailView(DetailsViewArgs);
	
	PythonOptionsData = NewObject<UNewPluginPythonOptionsData>();
	TSharedPtr<IDetailsView> PythonOptionsDetailView = EditModule.CreateDetailView(DetailsViewArgs);

	if ( !PluginWizardDefinition.IsValid() )
	{
		PluginWizardDefinition = MakeShared<FDefaultPluginWizardDefinition>(IsContentOnlyProject());
	}
	check(PluginWizardDefinition.IsValid());

	// Ensure that nothing is selected in the plugin wizard definition
	PluginWizardDefinition->ClearTemplateSelection();

	IPluginWizardDefinition* WizardDef = PluginWizardDefinition.Get();
	
	// Check if the Plugin Wizard is trying to make mods instead of generic plugins. This will slightly change in 4.26
	if (PluginWizardDefinition->IsMod())
	{
		AbsoluteGamePluginPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FPaths::ProjectModsDir());
		FPaths::MakePlatformFilename(AbsoluteGamePluginPath);
	}

	LastBrowsePath = AbsoluteGamePluginPath;
	PluginFolderPath = AbsoluteGamePluginPath;
	bIsPluginPathValid = true;

	// Create the list view and ensure that it exists
	GenerateListViewWidget();
	check(ListView.IsValid());

	TSharedPtr<SWidget> HeaderWidget = PluginWizardDefinition->GetCustomHeaderWidget();
	FText PluginNameTextHint = PluginWizardDefinition->IsMod() ? LOCTEXT("ModNameTextHint", "Mod Name") : LOCTEXT("PluginNameTextHint", "Plugin Name");
	
	const FMargin PaddingAmount(5.0f);

	TSharedRef<SVerticalBox> MainContent = SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.Padding(PaddingAmount)
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		
		// Custom header widget display
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(PaddingAmount)
		[
			HeaderWidget.IsValid() ? HeaderWidget.ToSharedRef() : SNullWidget::NullWidget
		]
		
		// Instructions
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(PaddingAmount)
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(PaddingAmount)
			.VAlign(VAlign_Center)
			.FillHeight(1.0f)
			[
				SNew(STextBlock)
				.Text(WizardDef, &IPluginWizardDefinition::GetInstructions)
				.AutoWrapText(true)
			]
		]
	]
	+SVerticalBox::Slot()
	.Padding(PaddingAmount)
	[
		// main list of plugins
		ListView.ToSharedRef()
	]
	+SVerticalBox::Slot()
	.AutoHeight()
	.Padding(PaddingAmount)
	.HAlign(HAlign_Center)
	[
		SAssignNew(FilePathBlock, SFilePathBlock)
		.OnBrowseForFolder(this, &SNewPluginWizard::OnBrowseButtonClicked)
		.FolderPath(this, &SNewPluginWizard::GetPluginDestinationPath)
		.Name(this, &SNewPluginWizard::GetCurrentPluginName)
		.NameHint(PluginNameTextHint)
		.OnFolderChanged(this, &SNewPluginWizard::OnFolderPathTextChanged)
		.OnNameChanged(this, &SNewPluginWizard::OnPluginNameTextChanged)
	];

	// Add the descriptor data object if it exists
	if (DescriptorData.IsValid() && DescriptorDetailView.IsValid())
	{
		DescriptorDetailView->SetObject(DescriptorData.Get());

		MainContent->AddSlot()
		.AutoHeight()
		.Padding(PaddingAmount)
		[
			DescriptorDetailView.ToSharedRef()
		];
	}

	// Add the python options data object if it exists
	if (PythonOptionsData.IsValid() && PythonOptionsDetailView.IsValid())
	{
		PythonOptionsDetailView->SetObject(PythonOptionsData.Get());

		MainContent->AddSlot()
		.AutoHeight()
		.Padding(PaddingAmount)
		[
			SNew(SBox)
			.Visibility_Lambda([this]()
			{
				FPluginTemplateDescription* Template = PluginWizardDefinition->GetSelectedTemplate().Get();
				return ((Template != nullptr) && !Template->PostCreatePythonScriptPath.IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
			})
			[
				PythonOptionsDetailView.ToSharedRef()
			]
		];
	}

	MainContent->AddSlot()
	.AutoHeight()
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(PaddingAmount)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SNewPluginWizard::OnEnginePluginCheckboxChanged)
			.IsChecked(this, &SNewPluginWizard::IsEnginePlugin)
			.Visibility_Lambda([this]()
			{
				FPluginTemplateDescription* Template = PluginWizardDefinition->GetSelectedTemplate().Get();
				return ((Template != nullptr) && Template->bCanBePlacedInEngine) ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.ToolTipText(LOCTEXT("EnginePluginButtonToolTip", "Toggles whether this plugin will be created in the current project or the engine directory."))
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EnginePluginCheckbox", "Is Engine Plugin"))
			]
		]
	];

	if (PluginWizardDefinition->CanShowOnStartup())
	{
		MainContent->AddSlot()
		.AutoHeight()
		.Padding(PaddingAmount)
		[
			SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(WizardDef, &IPluginWizardDefinition::OnShowOnStartupCheckboxChanged)
				.IsChecked(WizardDef, &IPluginWizardDefinition::GetShowOnStartupCheckBoxState)
				.ToolTipText(LOCTEXT("ShowOnStartupToolTip", "Toggles whether this wizard will show when the editor is launched."))
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowOnStartupCheckbox", "Show on Startup"))
				]
			]
		];
	}

	// Checkbox to show the plugin's content directory when the plugin is created
	MainContent->AddSlot()
	.AutoHeight()
	.Padding(PaddingAmount)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SAssignNew(ShowPluginContentDirectoryCheckBox, SCheckBox)
			.IsChecked(ECheckBoxState::Checked)
			.Visibility_Lambda([this]()
			{
				FPluginTemplateDescription* Template = PluginWizardDefinition->GetSelectedTemplate().Get();
				return ((Template != nullptr) && Template->bCanContainContent) ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.ToolTipText(LOCTEXT("ShowPluginContentDirectoryToolTip", "Shows the content directory after creation."))
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ShowPluginContentDirectoryText", "Show Content Directory"))
			]
		]
	];

	FText CreateButtonLabel = PluginWizardDefinition->IsMod() ? LOCTEXT("CreateModButtonLabel", "Create Mod") : LOCTEXT("CreatePluginButtonLabel", "Create Plugin");

	MainContent->AddSlot()
	.AutoHeight()
	.Padding(10)
	.HAlign(HAlign_Right)
	[
		SNew(SPrimaryButton)
		.IsEnabled(this, &SNewPluginWizard::CanCreatePlugin)
		.Text(CreateButtonLabel)
		.OnClicked(this, &SNewPluginWizard::OnCreatePluginClicked)
	];

	ChildSlot
	[
		MainContent
	];
}

void SNewPluginWizard::GenerateListViewWidget()
{
	// Get the source of the templates to use for the list view
	const TArray<TSharedRef<FPluginTemplateDescription>>& TemplateSource = PluginWizardDefinition->GetTemplatesSource();

	ListView = SNew(SListView<TSharedRef<FPluginTemplateDescription>>)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(&TemplateSource)
		.OnGenerateRow(this, &SNewPluginWizard::OnGenerateTemplateRow)
		.OnSelectionChanged(this, &SNewPluginWizard::OnTemplateSelectionChanged);
}

void SNewPluginWizard::GeneratePluginTemplateDynamicBrush(TSharedRef<FPluginTemplateDescription> InItem)
{
	if ( !InItem->PluginIconDynamicImageBrush.IsValid() )
	{
		// Plugin thumbnail image
		FString Icon128FilePath;
		PluginWizardDefinition->GetTemplateIconPath(InItem, Icon128FilePath);

		const FName BrushName(*Icon128FilePath);
		const FIntPoint Size = FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(BrushName);
		if ((Size.X > 0) && (Size.Y > 0))
		{
			InItem->PluginIconDynamicImageBrush = MakeShareable(new FSlateDynamicImageBrush(BrushName, FVector2D(Size.X, Size.Y)));
		}
	}
}

TSharedRef<ITableRow> SNewPluginWizard::OnGenerateTemplateTile(TSharedRef<FPluginTemplateDescription> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const float PaddingAmount = FPluginStyle::Get()->GetFloat("PluginTile.Padding");
	const float ThumbnailImageSize = FPluginStyle::Get()->GetFloat("PluginTile.ThumbnailImageSize");

	GeneratePluginTemplateDynamicBrush(InItem);

	return SNew(STableRow< TSharedRef<FPluginTemplateDescription> >, OwnerTable)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Padding(PaddingAmount)
			.ToolTipText(InItem->Description)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(PaddingAmount)
				[
					SNew(SVerticalBox)
					
					// Template thumbnail image
					+ SVerticalBox::Slot()
					.Padding(PaddingAmount)
					.AutoHeight()
					[
						SNew(SBox)
						.WidthOverride(ThumbnailImageSize)
						.HeightOverride(ThumbnailImageSize)
						[
							SNew(SImage)
							.Image(InItem->PluginIconDynamicImageBrush.IsValid() ? InItem->PluginIconDynamicImageBrush.Get() : nullptr)
						]
					]

					// Template name
					+ SVerticalBox::Slot()
					.Padding(PaddingAmount)
					.FillHeight(1.0)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.Padding(PaddingAmount)
						.HAlign(HAlign_Center)
						.FillWidth(1.0)
						[
							SNew(STextBlock)
							.Text(InItem->Name)
							.TextStyle(FPluginStyle::Get(), "PluginTile.DescriptionText")
							.AutoWrapText(true)
							.Justification(ETextJustify::Center)
						]
					]
				]
			]
		];
}

TSharedRef<ITableRow> SNewPluginWizard::OnGenerateTemplateRow(TSharedRef<FPluginTemplateDescription> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	const float PaddingAmount = FPluginStyle::Get()->GetFloat("PluginTile.Padding");
	const float ThumbnailImageSize = FPluginStyle::Get()->GetFloat("PluginTile.ThumbnailImageSize");
	const bool bTemplateEnabled = InItem->bSupportsContentOnlyProjects || !IsContentOnlyProject();

	GeneratePluginTemplateDynamicBrush(InItem);

	return SNew(STableRow< TSharedRef<FPluginTemplateDescription> >, OwnerTable)
		.IsEnabled(bTemplateEnabled)
		.ToolTipText_Lambda([bTemplateEnabled] { return !bTemplateEnabled ? LOCTEXT("TemplateDisabled", "This template is not available in Blueprint-only projects.") : FText::GetEmpty(); })
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Padding(PaddingAmount)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(PaddingAmount)
				[
					SNew(SHorizontalBox)
					
					// Template thumbnail image
					+ SHorizontalBox::Slot()
					.Padding(PaddingAmount)
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(ThumbnailImageSize)
						.HeightOverride(ThumbnailImageSize)
						[
							SNew(SImage)
							.Image(InItem->PluginIconDynamicImageBrush.IsValid() ? InItem->PluginIconDynamicImageBrush.Get() : nullptr)
						]
					]

					// Template name and description
					+ SHorizontalBox::Slot()
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(PaddingAmount)
						[
							SNew(STextBlock)
							.Text(InItem->Name)
							.TextStyle(FPluginStyle::Get(), "PluginTile.NameText")
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(PaddingAmount)
						[
							SNew(SRichTextBlock)
							.Text(InItem->Description)
							.TextStyle(FPluginStyle::Get(), "PluginTile.DescriptionText")
							.AutoWrapText(true)
						]
					]
				]
			]
		];
}


void SNewPluginWizard::OnTemplateSelectionChanged(TSharedPtr<FPluginTemplateDescription> InItem, ESelectInfo::Type SelectInfo)
{
	// Forward the set of selected items to the plugin wizard definition
	TSharedPtr<FPluginTemplateDescription> SelectedItem;

	if (ListView.IsValid())
	{
		if (ListView->GetSelectedItems().Num() > 0)
		{
			SelectedItem = ListView->GetSelectedItems()[0];
		}
	}

	if (PluginWizardDefinition.IsValid())
	{
		TSharedPtr<FPluginTemplateDescription> OldSelectedItem = PluginWizardDefinition->GetSelectedTemplate();

		if (OldSelectedItem != SelectedItem)
		{
			if (OldSelectedItem.IsValid())
			{
				OldSelectedItem->UpdatePathWhenTemplateUnselected(PluginFolderPath);
				OldSelectedItem->UpdatePluginNameTextWhenTemplateUnselected(PluginNameText);
			}

			PluginWizardDefinition->OnTemplateSelectionChanged(SelectedItem, SelectInfo);

			if (SelectedItem.IsValid())
			{
				SelectedItem->UpdatePathWhenTemplateSelected(PluginFolderPath);
				SelectedItem->UpdatePluginNameTextWhenTemplateSelected(PluginNameText);
			}
		}
	}

	ValidateFullPluginPath();
}

void SNewPluginWizard::OnFolderPathTextChanged(const FText& InText)
{
	PluginFolderPath = InText.ToString();
	FPaths::MakePlatformFilename(PluginFolderPath);
	ValidateFullPluginPath();
}

void SNewPluginWizard::OnPluginNameTextChanged(const FText& InText)
{
	PluginNameText = InText;
	ValidateFullPluginPath();
}

FReply SNewPluginWizard::OnBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if (MainFrameParentWindow.IsValid() && MainFrameParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = MainFrameParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		FString FolderName;
		const FString Title = LOCTEXT("NewPluginBrowseTitle", "Choose a plugin location").ToString();
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			Title,
			LastBrowsePath,
			FolderName
			);

		if (bFolderSelected)
		{
			LastBrowsePath = FolderName;
			OnFolderPathTextChanged(FText::FromString(FolderName));
		}
	}

	return FReply::Handled();
}

void SNewPluginWizard::ValidateFullPluginPath()
{
	// Check for issues with path
	bIsPluginPathValid = false;
	bool bIsNewPathValid = true;
	FText FolderPathError;

	if (!FPaths::ValidatePath(GetPluginDestinationPath().ToString(), &FolderPathError))
	{
		bIsNewPathValid = false;
	}

	if (bIsNewPathValid)
	{
		bool bFoundKnownPath = false;
		bool bPathIsInEngine = false;
		FString AbsolutePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*GetPluginDestinationPath().ToString());
		FPaths::MakePlatformFilename(AbsolutePath);

		if (AbsolutePath.StartsWith(AbsoluteGamePluginPath))
		{
			bFoundKnownPath = true;
			bPathIsInEngine = false;
		}
		else if (AbsolutePath.StartsWith(AbsoluteEnginePluginPath))
		{
			bFoundKnownPath = true;
			bPathIsInEngine = true;
		}
		else
		{
			// Path is neither engine nor game, so this path will be added to the additional plugin directories for
			// the project when the plugin is created
		}

		bIsSelectedPathInEngine = bPathIsInEngine;

		// Run this path by the current template to make sure it's OK with it
		TSharedPtr<FPluginTemplateDescription> Template = PluginWizardDefinition->GetSelectedTemplate();
		if (Template.IsValid() && bIsNewPathValid)
		{
			if (bFoundKnownPath && bPathIsInEngine && !Template->bCanBePlacedInEngine)
			{
				bIsNewPathValid = false;
				if (FApp::IsInstalled())
				{
					FolderPathError = LOCTEXT("TemplateCannotBeInEngine_InstalledBuild", "Plugins created in installed builds cannot be placed in the engine folder");
				}
				else
				{
					FolderPathError = LOCTEXT("TemplateCannotBeInEngine", "Plugins created from this template cannot be placed in the engine folder");
				}
			}
		}

		if (bIsNewPathValid && Template.IsValid())
		{
			bIsNewPathValid = Template->ValidatePathForPlugin(AbsolutePath, /*out*/ FolderPathError);
		}
	}

	bIsPluginPathValid = bIsNewPathValid;
	FilePathBlock->SetFolderPathError(FolderPathError);

	// Check for issues with name. Fail silently if text is empty
	FText PluginNameError;
	bIsPluginNameValid = !GetCurrentPluginName().IsEmpty() && FPluginUtils::ValidateNewPluginNameAndLocation(GetExpandedPluginName(), PluginFolderPath, &PluginNameError);
	FilePathBlock->SetNameError(PluginNameError);
}

bool SNewPluginWizard::CanCreatePlugin() const
{
	return bIsPluginPathValid && bIsPluginNameValid && PluginWizardDefinition->HasValidTemplateSelection() && !bIsWaitingForAssetDiscoveryToFinish;
}

FText SNewPluginWizard::GetPluginDestinationPath() const
{
	return FText::FromString(PluginFolderPath);
}

FText SNewPluginWizard::GetCurrentPluginName() const
{
	return PluginNameText;
}

FString SNewPluginWizard::GetExpandedPluginName() const
{
	TSharedPtr<FPluginTemplateDescription> Template = PluginWizardDefinition->GetSelectedTemplate();
	if (Template.IsValid())
	{
		TArray<FString> UPluginFiles;
		IFileManager::Get().FindFiles(UPluginFiles, *Template->OnDiskPath, TEXT("*.uplugin"));
		for (FString File : UPluginFiles)
		{
			File.RemoveFromEnd(TEXT(".uplugin"));
			if (File.Contains(TEXT("PLUGIN_NAME")))
			{
				return File.Replace(TEXT("PLUGIN_NAME"), *GetCurrentPluginName().ToString(), ESearchCase::CaseSensitive);
			}
		}
	}
	return GetCurrentPluginName().ToString();
}

ECheckBoxState SNewPluginWizard::IsEnginePlugin() const
{
	return bIsSelectedPathInEngine ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNewPluginWizard::OnEnginePluginCheckboxChanged(ECheckBoxState NewCheckedState)
{
	bool bNewEnginePluginState = NewCheckedState == ECheckBoxState::Checked;
	if (bIsSelectedPathInEngine != bNewEnginePluginState)
	{
		bIsSelectedPathInEngine = bNewEnginePluginState;
		if (bIsSelectedPathInEngine)
		{
			PluginFolderPath = AbsoluteEnginePluginPath;
		}
		else
		{
			PluginFolderPath = AbsoluteGamePluginPath;
		}

		ValidateFullPluginPath();
	}
}

FReply SNewPluginWizard::OnCreatePluginClicked()
{
	if (!ensure(!PluginFolderPath.IsEmpty() && !GetCurrentPluginName().IsEmpty()))
	{
		// Don't even try to assemble the path or else it may be relative to the binaries folder!
		return FReply::Unhandled();
	}

	TSharedPtr<FPluginTemplateDescription> Template = PluginWizardDefinition->GetSelectedTemplate();
	if (!ensure(Template.IsValid()))
	{
		return FReply::Unhandled();
	}

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (AssetToolsModule.Get().IsDiscoveringAssetsInProgress())
	{
		bIsWaitingForAssetDiscoveryToFinish = true;
		AssetToolsModule.Get().OpenDiscoveringAssetsDialog(IAssetTools::FOnAssetsDiscovered::CreateSP(this, &SNewPluginWizard::DiscoveringAssetsCalback));
		return FReply::Unhandled();
	}

	FString PluginName = GetExpandedPluginName();
	const bool bHasModules = PluginWizardDefinition->HasModules();
	
	FPluginUtils::FNewPluginParamsWithDescriptor CreationParams;
	CreationParams.TemplateFolders = PluginWizardDefinition->GetFoldersForSelection();
	CreationParams.NameToReplace = GetCurrentPluginName().ToString();
	CreationParams.Descriptor.bCanContainContent = Template->bCanContainContent;

	if (bHasModules)
	{
		CreationParams.Descriptor.Modules.Add(FModuleDescriptor(*PluginName, PluginWizardDefinition->GetPluginModuleDescriptor(), PluginWizardDefinition->GetPluginLoadingPhase()));
	}

	CreationParams.Descriptor.FriendlyName = PluginName;
	CreationParams.Descriptor.Version = 1;
	CreationParams.Descriptor.VersionName = TEXT("1.0");
	CreationParams.Descriptor.Category = TEXT("Other");

	PluginWizardDefinition->GetPluginIconPath(/*out*/ CreationParams.PluginIconPath);
	if (DescriptorData.IsValid())
	{
		CreationParams.Descriptor.CreatedBy = DescriptorData->CreatedBy;
		CreationParams.Descriptor.CreatedByURL = DescriptorData->CreatedByURL;
		CreationParams.Descriptor.Description = DescriptorData->Description;
		CreationParams.Descriptor.bIsBetaVersion = DescriptorData->bIsBetaVersion;
	}
	
	CreationParams.PostCreatePythonScriptPath = Template->PostCreatePythonScriptPath;
	CreationParams.PostCreatePythonScriptArguments = Template->PostCreatePythonScriptArguments;
	
	if (PythonOptionsData.IsValid())
	{
		CreationParams.PostCreatePythonScriptAdditionalArguments = PythonOptionsData->AdditionalArguments;
	}

	FText FailReason;
	FPluginUtils::FLoadPluginParams LoadParams;
	LoadParams.bEnablePluginInProject = true;
	LoadParams.bUpdateProjectPluginSearchPath = true;
	LoadParams.bSelectInContentBrowser = ShowPluginContentDirectoryCheckBox->IsChecked();
	LoadParams.OutFailReason = &FailReason;

	Template->CustomizeDescriptorBeforeCreation(CreationParams.Descriptor);


	TSharedPtr<IPlugin> NewPlugin = FPluginUtils::CreateAndLoadNewPlugin(PluginName, PluginFolderPath, CreationParams, LoadParams);
	const bool bSucceeded = NewPlugin.IsValid();


	PluginWizardDefinition->PluginCreated(PluginName, bSucceeded);

	if (bSucceeded)
	{
		// Let the template create additional assets / modify state after creation
		Template->OnPluginCreated(NewPlugin);

		// Notify that a new plugin has been created
		FPluginBrowserModule& PluginBrowserModule = FPluginBrowserModule::Get();
		PluginBrowserModule.BroadcastNewPluginCreated();

		FNotificationInfo Info(FText::Format(LOCTEXT("PluginCreatedSuccessfully", "'{0}' was created successfully."), FText::FromString(PluginName)));
		Info.bUseThrobber = false;
		Info.ExpireDuration = 8.0f;
		FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Success);

		OwnerTab.Pin()->RequestCloseTab();

		if (bHasModules)
		{
			FSourceCodeNavigation::OpenModuleSolution();
		}

		return FReply::Handled();
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, FailReason, LOCTEXT("UnableToCreatePlugin", "Unable to create plugin"));
		return FReply::Unhandled();
	}
}

void SNewPluginWizard::DiscoveringAssetsCalback()
{
	bIsWaitingForAssetDiscoveryToFinish = false;
}

#undef LOCTEXT_NAMESPACE
