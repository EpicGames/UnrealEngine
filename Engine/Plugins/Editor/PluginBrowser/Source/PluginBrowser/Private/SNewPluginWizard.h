// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IPluginWizardDefinition;
class SDockTab;
class UNewPluginDescriptorData;
class UNewPluginPythonOptionsData;
namespace ESelectInfo { enum Type : int; }
struct FPluginTemplateDescription;

class ITableRow;
class SFilePathBlock;
class STableViewBase;
enum class ECheckBoxState : uint8;

template <typename ItemType>
class SListView;

DECLARE_LOG_CATEGORY_EXTERN(LogPluginWizard, Log, All);

class SFilePathBlock;

/**
 * A wizard to create a new plugin
 */
class SNewPluginWizard : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNewPluginWizard){}
	SLATE_END_ARGS()

	/** Constructor */
	SNewPluginWizard();

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<SDockTab> InOwnerTab, TSharedPtr<IPluginWizardDefinition> InPluginWizardDefinition = nullptr);

private:
	/**
	 * Called when Folder Path textbox changes value
	 * @param InText The new Plugin Folder Path text
	 */
	void OnFolderPathTextChanged(const FText& InText);

	/**
	 * Called to generate a widget for the specified list item
	 * @param Item The template information for this row
	 * @param OwnerTable The table that owns these rows
	 * @return The widget for this template
	 */
	TSharedRef<ITableRow> OnGenerateTemplateRow(TSharedRef<FPluginTemplateDescription> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Called to generate a widget for the specified tile item
	 * @param Item The template information for this row
	 * @param OwnerTable The table that owns these rows
	 * @return The widget for this template
	 */
	TSharedRef<ITableRow> OnGenerateTemplateTile(TSharedRef<FPluginTemplateDescription> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Called when the template selection changes
	 */
	void OnTemplateSelectionChanged(TSharedPtr<FPluginTemplateDescription> InItem, ESelectInfo::Type SelectInfo);

	/**
	 * Called when Plugin Name textbox changes value
	 * @param InText The new Plugin name text
	 */
	void OnPluginNameTextChanged(const FText& InText);

	/** Handler for when the Browse button is clicked */
	FReply OnBrowseButtonClicked();

	/**
	 * Validates both the current path and plugin name as the final step in name
	 * validation requires a valid path.
	 */
	void ValidateFullPluginPath();

	/**
	 * Whether we are currently able to create a plugin
	 */
	bool CanCreatePlugin() const;

	/**
	 * Get the path where we will create a plugin
	 */
	FText GetPluginDestinationPath() const;

	/** Get the current name of the plugin */
	FText GetCurrentPluginName() const;

	/**
	 * Get the actual name of the plugin that will be created, derived from the selected template's
	 * .uplugin filename (replacing the PLUGIN_NAME token with the user-entered name). Falls back to
	 * the user-entered name if the template does not use the token.
	 */
	FString GetExpandedPluginName() const;

	/**
	 * Whether we will create a plugin in the engine directory
	 */
	ECheckBoxState IsEnginePlugin() const;

	/**
	 * Called when state of Engine plugin checkbox changes
	 * @param NewCheckedState New state of the checkbox
	 */
	void OnEnginePluginCheckboxChanged(ECheckBoxState NewCheckedState);

	/**
	 * This is where all the magic happens.
	 * Create actual plugin using parameters collected from other widgets
	 */
	FReply OnCreatePluginClicked();

	/**
	 * Called when asset discovering completes if the user attempts to create a new plugin while asset discovery is still in progress.
	 */
	void DiscoveringAssetsCalback();

	/**
	 * Generates the expected list view for the plugin wizard, based on the wizard's definition
	 */
	void GenerateListViewWidget();

	/**
	 * Generates the dynamic brush resource for a plugin template definition if it has not yet been created
	 */
	void GeneratePluginTemplateDynamicBrush(TSharedRef<FPluginTemplateDescription> InItem);

private:
	/** Additional user-defined descriptor data */
	TWeakObjectPtr<UNewPluginDescriptorData> DescriptorData;
	
	/** Additional user-defined descriptor data */
	TWeakObjectPtr<UNewPluginPythonOptionsData> PythonOptionsData;

	/** The current plugin wizard definition */
	TSharedPtr<IPluginWizardDefinition> PluginWizardDefinition;

	/** The list view for template selection */
	TSharedPtr<SListView<TSharedRef<FPluginTemplateDescription>>> ListView;

	/** Absolute path to game plugins directory so we don't have to convert it repeatedly */
	FString AbsoluteGamePluginPath;

	/** Absolute path to engine plugins directory so we don't have to convert it repeatedly */
	FString AbsoluteEnginePluginPath;

	/** Last Path used to browse, so that we know it will open dialog */
	FString LastBrowsePath;

	/** Path where you want to create the plugin*/
	FString PluginFolderPath;

	/** Name of the plugin you want to create*/
	FText PluginNameText;

	/** File Path widget that user will choose plugin location and name with */
	TSharedPtr<SFilePathBlock> FilePathBlock;

	/** Check box to show a plugin's content directory once a plugin has been successfully created */
	TSharedPtr<class SCheckBox> ShowPluginContentDirectoryCheckBox;

	/** Whether the path of the plugin entered is currently valid */
	bool bIsPluginPathValid;

	/** Whether the name of the plugin entered is currently valid */
	bool bIsPluginNameValid;

	/** Whether we want to create a plugin in the engine folder */
	bool bIsSelectedPathInEngine;

	/** Whether the asset discovery dialog is open and the user needs to wait on it */
	bool bIsWaitingForAssetDiscoveryToFinish;

	/** Tab that owns this wizard so that we can ask to close after completion */
	TWeakPtr<SDockTab> OwnerTab;
};
