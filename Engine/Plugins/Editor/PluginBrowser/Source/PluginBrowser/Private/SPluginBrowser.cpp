// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginBrowser.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/App.h"
#include "SPluginCategory.h"
#include "Widgets/Input/SComboButton.h"
#include "SPluginTileList.h"
#include "Widgets/SToolTip.h"
#include "Framework/Docking/TabManager.h"
#include "Interfaces/IPluginManager.h"
#include "PluginStyle.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "SPluginCategoryTree.h"
#include "Widgets/Input/SSearchBox.h"
#include "PluginBrowserModule.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "Interfaces/IProjectManager.h"
#include "ProjectDescriptor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/StyleColors.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Views/SListView.h" // IWYU pragma: keep
#include "Misc/ConfigCacheIni.h"

#define LOCTEXT_NAMESPACE "PluginsEditor"

SPluginBrowser::~SPluginBrowser()
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	for(auto& Pair: WatchDirectories)
	{
		DirectoryWatcherModule.Get()->UnregisterDirectoryChangedCallback_Handle(Pair.Key, Pair.Value);
	}

	FPluginBrowserModule::Get().OnNewPluginCreated().RemoveAll(this);
}

void SPluginBrowser::Construct( const FArguments& Args )
{
	OnRestartClicked = Args._OnRestartClicked;

	// Get the root directories which contain plugins
	TArray<FString> WatchDirectoryNames;
	WatchDirectoryNames.Add(FPaths::EnginePluginsDir());
	if (FPaths::DirectoryExists(FPaths::EnterprisePluginsDir()))
	{
		WatchDirectoryNames.Add(FPaths::EnterprisePluginsDir());
	}
	if(FApp::HasProjectName())
	{
		const FString ProjectPluginsDir = FPaths::ProjectPluginsDir();
		if (FPaths::DirectoryExists(ProjectPluginsDir))
		{
			WatchDirectoryNames.Add(ProjectPluginsDir);
		}
		const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
		if (Project != nullptr)
		{
			for (const FString& Path : Project->GetAdditionalPluginDirectories())
			{
				if (FPaths::DirectoryExists(Path))
				{
					WatchDirectoryNames.Add(Path);
				}
			}
		}
	}

	// Add watchers for any change events on those directories
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	for(const FString& WatchDirectoryName: WatchDirectoryNames)
	{
		FDelegateHandle Handle;
		if(DirectoryWatcherModule.Get()->RegisterDirectoryChangedCallback_Handle(WatchDirectoryName, IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &SPluginBrowser::OnPluginDirectoryChanged), Handle, IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges))
		{
			WatchDirectories.Add(WatchDirectoryName, Handle);
		}
	}

	FPluginBrowserModule::Get().OnNewPluginCreated().AddSP(this, &SPluginBrowser::OnNewPluginCreated);

	RegisterActiveTimer (0.f, FWidgetActiveTimerDelegate::CreateSP (this, &SPluginBrowser::TriggerBreadcrumbRefresh));

	struct Local
	{
		static void PluginToStringArray(const IPlugin* Plugin, OUT TArray< FString >& StringArray)
		{
			const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
			StringArray.Add(Plugin->GetFriendlyName());
			StringArray.Add(Descriptor.Description);
		}
	};

	// Setup text filtering
	PluginTextFilter = MakeShareable( new FPluginTextFilter( FPluginTextFilter::FItemToStringArray::CreateStatic( &Local::PluginToStringArray ) ) );

	const float PaddingAmount = 2.0f;

	PluginCategories = SNew( SPluginCategoryTree, SharedThis( this ) );

	FMenuBuilder DetailViewOptions(true, nullptr);
	DetailViewOptions.AddMenuEntry(
		LOCTEXT("ShowOnlyEnabled", "Show Only Enabled"),
		LOCTEXT("ShowOnlyEnabled_ToolTip", "Displays only the plugins which are enabled"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
			{
				PluginCategories->ToggleFilterType(SPluginCategoryTree::EFilterType::OnlyEnabled);
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() -> bool
			{
				return PluginCategories->IsFilterEnabled(SPluginCategoryTree::EFilterType::OnlyEnabled);
			})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	DetailViewOptions.AddMenuEntry(
		LOCTEXT("ShowOnlyDisabled", "Show Only Disabled"),
		LOCTEXT("ShowOnlyDisabled_ToolTip", "Displays only the plugins which are disabled"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]
			{
				PluginCategories->ToggleFilterType(SPluginCategoryTree::EFilterType::OnlyDisabled);
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() -> bool
			{
				return PluginCategories->IsFilterEnabled(SPluginCategoryTree::EFilterType::OnlyDisabled);
			})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	TSharedRef<SBorder> MainContent = SNew( SBorder )
	.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
	.Padding(0.f)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			// Add Plugin button
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(12, 7, 0, 7))
			.AutoWidth()
			[
				SNew(SButton)
				.ToolTip(SNew(SToolTip).Text(LOCTEXT("NewPluginEnabled", "Click here to open the Plugin Creator dialog.")))
				.ContentPadding(FMargin(0, 5.f, 0, 4.f))
				.OnClicked(this, &SPluginBrowser::HandleNewPluginButtonClicked)
				.IsEnabled_Static([]()
				{
					bool bCreateEnabled = true;
					GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanCreatePluginsFromBrowser"), bCreateEnabled, GEditorIni);
					return bCreateEnabled;
				})
				.Content()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
						.ColorAndOpacity(FStyleColors::AccentGreen)
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(3, 0, 0, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "SmallButtonText")
						.Text(LOCTEXT("NewPluginLabel", "Add"))
					]
				]
			]

			// Plugin Directories button
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(7, 7, 18, 7))
			.AutoWidth()
			[
				SNew(SButton)
				.ToolTip(SNew(SToolTip).Text(LOCTEXT("PluginDirectoriesTooltip", "Click here to configure additional plugin search directories.")))
				.OnClicked_Static([]() -> FReply
				{
					FGlobalTabmanager::Get()->TryInvokeTab(FPluginBrowserModule::ExternalDirectoriesTabName);
					return FReply::Handled();
				})
				.ContentPadding(FMargin(0, 5.f, 0, 4.f))
				.Content()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FPluginStyle::Get()->GetBrush("Plugins.TabIcon"))
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(3, 0, 0, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "SmallButtonText")
						.Text(LOCTEXT("PluginDirectoriesLabel", "Plugin Directories"))
					]
				]
			]

			// Search Box
			+ SHorizontalBox::Slot()
			.Padding(0, 7, 0, 7)
			[
				SAssignNew(SearchBoxPtr, SSearchBox)
				.OnTextChanged(this, &SPluginBrowser::SearchBox_OnPluginSearchTextChanged)
			]

			// Settings button
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(14.f, 0.f, 14.f, 0.f))
			[
				SNew(SComboButton)
				.ContentPadding(0)
				.HasDownArrow(false)
				.ForegroundColor(FSlateColor::UseForeground())
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
				.MenuContent()
				[
					DetailViewOptions.MakeWidget()

				]
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Toolbar.Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(5, 0, 0, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NormalText")
						.Text(LOCTEXT("SettingsLabel", "Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]
		+SVerticalBox::Slot()
		[
			SNew( SSplitter )
			.Style(FAppStyle::Get(), "SplitterPanel")
			+SSplitter::Slot()
			.Value(.35f)
			[
				SNew(SBorder)
				.Padding(8.0f)
				.BorderImage(FPluginStyle::Get()->GetBrush("Plugins.ListBorder"))
				[
					PluginCategories.ToSharedRef()
				]
			]
			+SSplitter::Slot()
			[
				SNew( SVerticalBox )

				+SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, PaddingAmount))
				.AutoHeight()
				[
					SNew( SHorizontalBox )

					+SHorizontalBox::Slot()
					.Padding( PaddingAmount )
					[
						SAssignNew( BreadcrumbTrail, SBreadcrumbTrail<TSharedPtr<FPluginCategory>> )
						.DelimiterImage( FAppStyle::Get().GetBrush( "Icons.ChevronRight" ) ) 
						.ShowLeadingDelimiter( false )
						.OnCrumbClicked( this, &SPluginBrowser::BreadcrumbTrail_OnCrumbClicked )
					]
				]

				+SVerticalBox::Slot()
				[
					SAssignNew(PluginList, SPluginTileList, SharedThis(this))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(18.0f, 20.0f, 18.0f, 16.0f))
				[
					SNew(SWarningOrErrorBox)
					.Visibility(this, &SPluginBrowser::HandleRestartNoticeVisibility)
					.MessageStyle(EMessageStyle::Warning)
					.Message(LOCTEXT("PluginSettingsRestartNotice", "You must restart Unreal Editor for your changes to take effect."))
					[
						SNew(SButton)
						.OnClicked(this, &SPluginBrowser::HandleRestartButtonClicked)
						.TextStyle(FAppStyle::Get(), "NormalText")
						.Text(LOCTEXT("PluginSettingsRestartEditor", "Restart Now"))
					]
				]
			]
		]
	];
	

	ChildSlot
	[
		MainContent
	];
}

void SPluginBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (!bInitialFocusTaken)
	{
		bInitialFocusTaken = FSlateApplication::Get().SetKeyboardFocus(SearchBoxPtr);
	}
}

EVisibility SPluginBrowser::HandleRestartNoticeVisibility() const
{
	return FPluginBrowserModule::Get().ShowPendingRestart() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SPluginBrowser::HandleRestartButtonClicked() const
{
	if (ensure(OnRestartClicked.IsBound()))
	{
		return OnRestartClicked.Execute();
	}

	return FReply::Unhandled();
}

void SPluginBrowser::SearchBox_OnPluginSearchTextChanged( const FText& NewText )
{
	PluginTextFilter->SetRawFilterText( NewText );
	SearchBoxPtr->SetError( PluginTextFilter->GetFilterErrorText() );
}


TSharedPtr< FPluginCategory > SPluginBrowser::GetSelectedCategory() const
{
	return PluginCategories.IsValid() ? PluginCategories->GetSelectedCategory() : nullptr;
}


void SPluginBrowser::OnCategorySelectionChanged()
{
	if( PluginList.IsValid() )
	{
		PluginList->SetNeedsRefresh();
	}

	// Breadcrumbs will need to be refreshed
	RegisterActiveTimer (0.f, FWidgetActiveTimerDelegate::CreateSP (this, &SPluginBrowser::TriggerBreadcrumbRefresh));
}

void SPluginBrowser::SetNeedsRefresh()
{
	if( PluginList.IsValid() )
	{
		PluginList->SetNeedsRefresh();
	}

	if( PluginCategories.IsValid() )
	{
		PluginCategories->SetNeedsRefresh();
	}

	// Breadcrumbs will need to be refreshed
	RegisterActiveTimer (0.f, FWidgetActiveTimerDelegate::CreateSP (this, &SPluginBrowser::TriggerBreadcrumbRefresh));
}

void SPluginBrowser::OnPluginDirectoryChanged(const TArray<struct FFileChangeData>&)
{
	if(UpdatePluginsTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(UpdatePluginsTimerHandle.ToSharedRef());
	}
	UpdatePluginsTimerHandle = RegisterActiveTimer(2.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SPluginBrowser::UpdatePluginsTimerCallback));
}

void SPluginBrowser::OnNewPluginCreated()
{
	if (UpdatePluginsTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(UpdatePluginsTimerHandle.ToSharedRef());
	}
	UpdatePluginsTimerHandle = RegisterActiveTimer(2.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SPluginBrowser::UpdatePluginsTimerCallback));
}

EActiveTimerReturnType SPluginBrowser::UpdatePluginsTimerCallback(double InCurrentTime, float InDeltaTime)
{
	IPluginManager::Get().RefreshPluginsList();
	SetNeedsRefresh();
	return EActiveTimerReturnType::Stop;
}

EActiveTimerReturnType SPluginBrowser::TriggerBreadcrumbRefresh(double InCurrentTime, float InDeltaTime)
{
	RefreshBreadcrumbTrail();
	return EActiveTimerReturnType::Stop;
}

void SPluginBrowser::RefreshBreadcrumbTrail()
{
	// Update breadcrumb trail
	if( BreadcrumbTrail.IsValid() )
	{
		TSharedPtr<FPluginCategory> SelectedCategory = PluginCategories->GetSelectedCategory();

		// Build up the list of categories, starting at the selected node and working our way backwards.
		TArray<TSharedPtr<FPluginCategory>> CategoryPath;
		if(SelectedCategory.IsValid())
		{
			for(TSharedPtr<FPluginCategory> NextCategory = SelectedCategory; NextCategory.IsValid(); NextCategory = NextCategory->ParentCategory.Pin())
			{
				CategoryPath.Insert( NextCategory, 0 );
			}
		}

		// Fill in the crumbs
		BreadcrumbTrail->ClearCrumbs();
		for(TSharedPtr<FPluginCategory>& Category: CategoryPath)
		{
			BreadcrumbTrail->PushCrumb( Category->DisplayName, Category );
		}
	}
}


void SPluginBrowser::BreadcrumbTrail_OnCrumbClicked( const TSharedPtr<FPluginCategory>& Category )
{
	if( PluginCategories.IsValid() )
	{
		PluginCategories->SelectCategory( Category );
	}
}

FReply SPluginBrowser::HandleNewPluginButtonClicked() const
{
	FGlobalTabmanager::Get()->TryInvokeTab( FPluginBrowserModule::PluginCreatorTabName );

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
