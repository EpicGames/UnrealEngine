// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TextFilter.h"
#include "Widgets/SCompoundWidget.h"
#include "PluginBrowserPrivate.h"

class FPluginCategory;
template <typename ItemType> class SBreadcrumbTrail;

class FActiveTimerHandle;
class SPluginCategoryTree;

/**
 * Implementation of main plugin editor Slate widget
 */
class SPluginBrowser : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(FReply, FReplyDelegate);

	SLATE_BEGIN_ARGS( SPluginBrowser )
	{
	}

		SLATE_EVENT(FReplyDelegate, OnRestartClicked)

	SLATE_END_ARGS()

	/** Destructor */
	virtual ~SPluginBrowser();

	/** Widget constructor */
	void Construct( const FArguments& Args );

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** @return Returns the plugin text filter object, so that child widgets can subscribe to find out about changes */
	FPluginTextFilter& GetPluginTextFilter()
	{
		return *PluginTextFilter;
	}

	/** @return Returns the currently selected category */
	TSharedPtr< class FPluginCategory > GetSelectedCategory() const;

	/** Called when the selected category changes so we can invalidate the list */
	void OnCategorySelectionChanged();

	/** Refresh the whole window */
	void SetNeedsRefresh();

private:

	/** Called with notification that one of the plugin directories has changed */
	void OnPluginDirectoryChanged(const TArray<struct FFileChangeData>& );

	/** Called with notification that a new plugin has been created */
	void OnNewPluginCreated();

	/** Timer callback for when */
	EActiveTimerReturnType UpdatePluginsTimerCallback(double InCurrentTime, float InDeltaTime);

	/** @return Is the "restart required" notice visible? */
	EVisibility HandleRestartNoticeVisibility() const;

	/** Handle the "restart now" button being clicked */
	FReply HandleRestartButtonClicked() const;

	/** Called when the text in the search box was changed */
	void SearchBox_OnPluginSearchTextChanged( const FText& NewText );

	/** Called when a breadcrumb is clicked on the breadcrumb trail */
	void BreadcrumbTrail_OnCrumbClicked( const TSharedPtr<FPluginCategory>& CrumbData );

	/** Called to refresh the breadcrumb trail immediately */
	void RefreshBreadcrumbTrail();

	/** One-off active timer to trigger a refresh of the breadcrumb trail as needed */
	EActiveTimerReturnType TriggerBreadcrumbRefresh(double InCurrentTime, float InDeltaTime);

	/** Handle the "new plugin" button being clicked */
	FReply HandleNewPluginButtonClicked() const;

private:
	/** Delegate to call when the restart button is clicked. */
	FReplyDelegate OnRestartClicked;

	/** Handles to the directory changed delegates */
	TMap<FString, FDelegateHandle> WatchDirectories;

	/** Handles to the directory changed delegates */
	TSharedPtr<FActiveTimerHandle> UpdatePluginsTimerHandle;

	/** The plugin categories widget */
	TSharedPtr< class SPluginCategoryTree > PluginCategories;

	/** The plugin list widget */
	TSharedPtr< class SPluginTileList > PluginList;

	/** The plugin search box widget */
	TSharedPtr< class SSearchBox > SearchBoxPtr;

	/** Text filter object for typing in filter text to the search box */
	TSharedPtr< FPluginTextFilter > PluginTextFilter;

	/** Breadcrumb trail widget for the currently selected category */
	TSharedPtr< SBreadcrumbTrail< TSharedPtr< FPluginCategory > > > BreadcrumbTrail;

	bool bInitialFocusTaken = false;
};

