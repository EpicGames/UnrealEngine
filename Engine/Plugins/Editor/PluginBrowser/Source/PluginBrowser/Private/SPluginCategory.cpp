// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginCategory.h"
#include "PluginStyle.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "PluginCategoryTreeItem"


void SPluginCategory::Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FPluginCategory>& InCategory)
{
	Category = InCategory;

	const float CategoryIconSize = FPluginStyle::Get()->GetFloat("CategoryTreeItem.IconSize");
	const float PaddingAmount = FPluginStyle::Get()->GetFloat("CategoryTreeItem.PaddingAmount");;

	// Figure out which font size to use
	const auto bIsRootItem = !Category->ParentCategory.IsValid();
	const auto bIsAllPlugins = bIsRootItem && !Category->SubCategories.Num();

	auto PluginCountLambda = [&]() -> FText {
		return FText::Format(LOCTEXT("NumberOfPluginsWrapper", "{0}"), FText::AsNumber(Category->Plugins.Num()));
	};

	FName BorderPadding = (bIsRootItem ? (bIsAllPlugins ? "CategoryTreeItem.Root.AllPluginsBackgroundPadding" : "CategoryTreeItem.Root.BackgroundPadding") : "CategoryTreeItem.BackgroundPadding");

	TSharedRef<SWidget> WidgetContent = 
		SNew( SBorder )
		.BorderImage( FPluginStyle::Get()->GetBrush(bIsRootItem ? "CategoryTreeItem.Root.BackgroundBrush" : "CategoryTreeItem.BackgroundBrush") )
		.Padding(FPluginStyle::Get()->GetMargin(BorderPadding) )
		[
			SNew( SHorizontalBox )

			// Icon image
			+SHorizontalBox::Slot()
			.Padding( PaddingAmount )
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.WidthOverride( CategoryIconSize )
				.HeightOverride( CategoryIconSize )
				[
					SNew( SImage )
					.Image( this, &SPluginCategory::GetIconBrush )
				]
			]

			// Category name
			+SHorizontalBox::Slot()
			.Padding( PaddingAmount )
			.VAlign(VAlign_Center)
			[
				SNew( STextBlock )
				.Text( Category->DisplayName )
				.TextStyle( FPluginStyle::Get(), bIsRootItem ? "CategoryTreeItem.Root.Text" : "CategoryTreeItem.Text" )
			]
			
			// Plugin count
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( PaddingAmount, PaddingAmount, PaddingAmount + 8.f, PaddingAmount) // Extra padding on the right
			.VAlign(VAlign_Center)
			[
				SNew( STextBlock )

				// Only display if at there is least one plugin is in this category
				.Visibility( Category->Plugins.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed )

				.Text_Lambda( PluginCountLambda )
				.TextStyle( FPluginStyle::Get(), bIsRootItem ? "CategoryTreeItem.Root.PluginCountText" : "CategoryTreeItem.PluginCountText" )
			]
		];

	this->ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		[
			SNew(SExpanderArrow, SharedThis(this) )
			.IndentAmount(0)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			WidgetContent
		]
	];

	STableRow< TSharedPtr<FPluginCategory> >::ConstructInternal(
		STableRow::FArguments().Style(&FAppStyle::Get(), "SimpleTableView.Row")
		.ShowSelection(true),
		InOwnerTableView
		);
}


const FSlateBrush* SPluginCategory::GetIconBrush() const
{
	if(Category->Name == TEXT("All"))
	{
		return FPluginStyle::Get()->GetBrush("Plugins.TabIcon");
	}
	else
	{
		return nullptr;
	}
}


#undef LOCTEXT_NAMESPACE
