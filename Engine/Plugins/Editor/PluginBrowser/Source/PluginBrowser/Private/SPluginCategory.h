// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;

class IPlugin;
struct FSlateBrush;

/**
 * Represents a category in the plugin category tree
 */
class FPluginCategory
{
public:
	/** Parent category item or NULL if this is a root category */
	TWeakPtr<FPluginCategory> ParentCategory;

	/** Name of the category */
	FString Name;

	/** Display name of the category */
	FText DisplayName;

	/** Child categories */
	TArray<TSharedPtr<FPluginCategory>> SubCategories;

	/** Plugins in this category */
	TArray<TSharedRef<IPlugin>> Plugins;

	/** Constructor for FPluginCategory */
	FPluginCategory(TSharedPtr<FPluginCategory> InParentCategory, const FString& InName, const FText& InDisplayName)
		: ParentCategory(InParentCategory),
		  Name(InName),
		  DisplayName(InDisplayName)
	{
	}

	/** Finds a sub-category by name */
	TSharedPtr<FPluginCategory> FindSubCategory(const FString& SubCategoryName)
	{
		for (TSharedPtr<FPluginCategory>& SubCategory : SubCategories)
		{
			if (SubCategory->Name == SubCategoryName)
			{
				return SubCategory;
			}
		}
		return TSharedPtr<FPluginCategory>();
	}
};

/**
 * Widget that represents a single category in the category tree view
 */
class SPluginCategory : public STableRow< TSharedPtr<FPluginCategory>>
{
	SLATE_BEGIN_ARGS( SPluginCategory )
	{
	}

	SLATE_END_ARGS()

	/** Widget constructor */
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FPluginCategory>& Item);

private:

	/** @return Gets the icon brush to use for this item's current state */
	const FSlateBrush* GetIconBrush() const;

private:

	/** The item we're representing the in tree */
	TSharedPtr<FPluginCategory> Category;
};

