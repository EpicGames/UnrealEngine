// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginCategoryTree.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/ConfigCacheIni.h"
#include "SPluginCategory.h"
#include "SPluginBrowser.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "PluginCategories"



void SPluginCategoryTree::Construct( const FArguments& Args, const TSharedRef< SPluginBrowser > Owner )
{
	OwnerWeak = Owner;

	FilterType = EFilterType::None;

	// Create the root categories
	AllCategory = MakeShareable(new FPluginCategory(NULL, TEXT("All"), LOCTEXT("AllCategoryName", "All Plugins")));
	BuiltInCategory = MakeShareable(new FPluginCategory(NULL, TEXT("Built-In"), LOCTEXT("BuiltInCategoryName", "Built-In")));
	ExternalCategory = MakeShareable(new FPluginCategory(NULL, TEXT("External"), LOCTEXT("ExternalCategoryName", "External")));
	InstalledCategory = MakeShareable(new FPluginCategory(NULL, TEXT("Installed"), LOCTEXT("InstalledCategoryName", "Installed")));
	ProjectCategory = MakeShareable(new FPluginCategory(NULL, TEXT("Project"), LOCTEXT("ProjectCategoryName", "Project")));
	ModCategory = MakeShareable(new FPluginCategory(NULL, TEXT("Mods"), LOCTEXT("ModsCategoryName", "Mods")));

	// Create the tree view control
	TreeView =
		SNew( STreeView<TSharedPtr<FPluginCategory>> )
		// For now we only support selecting a single folder in the tree
		.SelectionMode( ESelectionMode::Single )
		.ClearSelectionOnClick( false )		// Don't allow user to select nothing.  We always expect a category to be selected!
		.TreeItemsSource( &RootCategories )
		.OnGenerateRow( this, &SPluginCategoryTree::PluginCategoryTreeView_OnGenerateRow ) 
		.OnGetChildren( this, &SPluginCategoryTree::PluginCategoryTreeView_OnGetChildren )
		.TreeViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
		.OnSelectionChanged( this, &SPluginCategoryTree::PluginCategoryTreeView_OnSelectionChanged )
		;

	RebuildAndFilterCategoryTree();

	ChildSlot.AttachWidget( TreeView.ToSharedRef() );
}


SPluginCategoryTree::~SPluginCategoryTree()
{
}


/** @return Gets the owner of this list */
SPluginBrowser& SPluginCategoryTree::GetOwner()
{
	return *OwnerWeak.Pin();
}


static void ResetCategories(TArray<TSharedPtr<FPluginCategory>>& Categories)
{
	for (const TSharedPtr<FPluginCategory>& Category : Categories)
	{
		ResetCategories(Category->SubCategories);
		Category->Plugins.Reset();
		Category->SubCategories.Reset();
	}
}

void SPluginCategoryTree::RebuildAndFilterCategoryTree()
{
	// Get the path to the first currently selected category
	TArray<FString> SelectCategoryPath;
	if (auto SelectedItems = TreeView->GetSelectedItems(); SelectedItems.Num() > 0)
	{
		TSharedPtr<FPluginCategory> SelectedItem = SelectedItems[0];
		for (const FPluginCategory* Category = SelectedItem.Get(); Category != nullptr; Category = Category->ParentCategory.Pin().Get())
		{
			SelectCategoryPath.Insert(Category->Name, 0);
		}
	}

	// Clear the list of plugins in each current category
	ResetCategories(RootCategories);

	// Add all the known plugins into categories
	for(TSharedRef<IPlugin> Plugin: IPluginManager::Get().GetDiscoveredPlugins())
	{
		if (Plugin->IsHidden())
		{
			continue;
		}

		switch (FilterType)
		{
		case SPluginCategoryTree::EFilterType::None:
			break;
		case SPluginCategoryTree::EFilterType::OnlyEnabled:
			if (!Plugin->IsEnabled())
			{
				continue;
			}
			break;
		case SPluginCategoryTree::EFilterType::OnlyDisabled:
			if (Plugin->IsEnabled())
			{
				continue;
			}
			break;
		}

		// Figure out which base category this plugin belongs in
		TSharedPtr<FPluginCategory> RootCategory;
		if (Plugin->GetType() == EPluginType::Mod)
		{
			RootCategory = ModCategory;
		}
		else if (Plugin->GetType() == EPluginType::External)
		{
			RootCategory = ExternalCategory;
		}
		else if(Plugin->GetDescriptor().bInstalled)
		{
			RootCategory = InstalledCategory;
		}
		else if(Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
		{
			RootCategory = BuiltInCategory;
		}
		else
		{
			RootCategory = ProjectCategory;
		}

		// Get the subcategory for this plugin
		FString CategoryName = Plugin->GetDescriptor().Category;
		if(CategoryName.IsEmpty())
		{
			CategoryName = TEXT("Other");
		}

		// Locate this category at the level we're at in the hierarchy
		TSharedPtr<FPluginCategory> FoundCategory = NULL;
		for(TSharedPtr<FPluginCategory> TestCategory: RootCategory->SubCategories)
		{
			if(TestCategory->Name == CategoryName)
			{
				FoundCategory = TestCategory;
				break;
			}
		}

		if( !FoundCategory.IsValid() )
		{
			//@todo Allow for properly localized category names [3/7/2014 justin.sargent]
			FoundCategory = MakeShareable(new FPluginCategory(RootCategory, CategoryName, FText::FromString(CategoryName)));
			RootCategory->SubCategories.Add( FoundCategory );
		}
			
		// Associate the plugin with the category
		FoundCategory->Plugins.Add(Plugin);

		TSharedPtr<FPluginCategory> ParentCategory = FoundCategory->ParentCategory.Pin();
		while (ParentCategory.IsValid())
		{
			ParentCategory->Plugins.Add(Plugin);
			ParentCategory = ParentCategory->ParentCategory.Pin();
		}

		// Add the plugin in the "All" category
		if (AllCategory.IsValid())
		{
			AllCategory->Plugins.Add(Plugin);
		}
	}

	// Remove any empty categories, keeping track of which items are still selected
	for(TSharedPtr<FPluginCategory> RootCategory: RootCategories)
	{
		for(int32 Idx = 0; Idx < RootCategory->SubCategories.Num(); Idx++)
		{
			if(RootCategory->SubCategories[Idx]->Plugins.Num() == 0)
			{
				RootCategory->SubCategories.RemoveAt(Idx);
			}
		}
	}

	// Resolve the path to the category to select
	TSharedPtr<FPluginCategory> SelectCategory;
	if (SelectCategoryPath.Num() > 0)
	{
		for (TSharedPtr<FPluginCategory> RootCategory : RootCategories)
		{
			if (RootCategory->Name == SelectCategoryPath[0])
			{
				SelectCategory = RootCategory;
				for (int Idx = 1; Idx < SelectCategoryPath.Num(); Idx++)
				{
					TSharedPtr<FPluginCategory> SubCategory = SelectCategory->FindSubCategory(SelectCategoryPath[Idx]);
					if (!SubCategory.IsValid())
					{
						break;
					}
					SelectCategory = SubCategory;
				}
				break;
			}
		}
	}

	// Build the new list of root plugin categories
	RootCategories.Reset();
	if(ModCategory->SubCategories.Num() > 0 || ModCategory->Plugins.Num() > 0)
	{
		RootCategories.Add(ModCategory);
	}
	if(ProjectCategory->SubCategories.Num() > 0 || ProjectCategory->Plugins.Num() > 0)
	{
		RootCategories.Add(ProjectCategory);
	}
	if(InstalledCategory->SubCategories.Num() > 0 || InstalledCategory->Plugins.Num() > 0)
	{
		RootCategories.Add(InstalledCategory);
	}
	if (ExternalCategory->SubCategories.Num() > 0 || ExternalCategory->Plugins.Num() > 0)
	{
		RootCategories.Add(ExternalCategory);
	}
	if(BuiltInCategory->SubCategories.Num() > 0 || BuiltInCategory->Plugins.Num() > 0)
	{
		RootCategories.Add(BuiltInCategory);
	}
	if (RootCategories.Num() > 0)
	{
		RootCategories.Insert(AllCategory, 0);
	}

	// Optionally hide some categories
	{
		TArray<FString> HiddenCategoryNames;
		GConfig->GetArray(TEXT("EditorSettings"), TEXT("HidePluginCategoriesFromBrowser"), HiddenCategoryNames, GEditorIni);

		for (TArray<TSharedPtr<FPluginCategory>>::TIterator CategoryIter = RootCategories.CreateIterator(); CategoryIter; ++CategoryIter)
		{
			if (HiddenCategoryNames.Contains((*CategoryIter)->Name))
			{
				CategoryIter.RemoveCurrent();
			}
		}
	}

	// Sort every single category alphabetically
	for(TSharedPtr<FPluginCategory> RootCategory: RootCategories)
	{
		RootCategory->SubCategories.Sort([](const TSharedPtr<FPluginCategory>& A, const TSharedPtr<FPluginCategory>& B) -> bool { return A->DisplayName.CompareTo(B->DisplayName) < 0; });
	}

	// Expand all the root categories by default
	for(TSharedPtr<FPluginCategory> RootCategory: RootCategories)
	{
		TreeView->SetItemExpansion(RootCategory, true);
	}

	// Refresh the view
	TreeView->RequestTreeRefresh();

	// Make sure we have something selected
	if (RootCategories.Num() > 0)
	{
		if (SelectCategory.IsValid())
		{
			TreeView->SetSelection(SelectCategory);
		}
		else
		{
			TreeView->SetSelection(AllCategory);
		}
	}
}

TSharedRef<ITableRow> SPluginCategoryTree::PluginCategoryTreeView_OnGenerateRow( TSharedPtr<FPluginCategory> Item, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(SPluginCategory, OwnerTable, Item.ToSharedRef());
}


void SPluginCategoryTree::PluginCategoryTreeView_OnGetChildren(TSharedPtr<FPluginCategory> Item, TArray<TSharedPtr<FPluginCategory>>& OutChildren )
{
	OutChildren.Append(Item->SubCategories);
}


void SPluginCategoryTree::PluginCategoryTreeView_OnSelectionChanged(TSharedPtr<FPluginCategory> Item, ESelectInfo::Type SelectInfo )
{
	// Selection changed, which may affect which plugins are displayed in the list.  We need to invalidate the list.
	OwnerWeak.Pin()->OnCategorySelectionChanged();
}


TSharedPtr<FPluginCategory> SPluginCategoryTree::GetSelectedCategory() const
{
	if( TreeView.IsValid() )
	{
		auto SelectedItems = TreeView->GetSelectedItems();
		if( SelectedItems.Num() > 0 )
		{
			const auto& SelectedCategoryItem = SelectedItems[ 0 ];
			return SelectedCategoryItem;
		}
	}

	return NULL;
}

void SPluginCategoryTree::SelectCategory( const TSharedPtr<FPluginCategory>& CategoryToSelect )
{
	if( ensure( TreeView.IsValid() ) )
	{
		TreeView->SetSelection( CategoryToSelect );
	}
}

bool SPluginCategoryTree::IsItemExpanded( const TSharedPtr<FPluginCategory> Item ) const
{
	return TreeView->IsItemExpanded( Item );
}

void SPluginCategoryTree::SetNeedsRefresh()
{
	RegisterActiveTimer (0.f, FWidgetActiveTimerDelegate::CreateSP (this, &SPluginCategoryTree::TriggerCategoriesRefresh));
}

bool SPluginCategoryTree::IsFilterEnabled(EFilterType FilterValue) const
{
	return (FilterType == FilterValue);
}

void SPluginCategoryTree::ToggleFilterType(EFilterType FilterValue)
{
	FilterType = IsFilterEnabled(FilterValue) ? EFilterType::None : FilterValue;
	SetNeedsRefresh();
}

EActiveTimerReturnType SPluginCategoryTree::TriggerCategoriesRefresh(double InCurrentTime, float InDeltaTime)
{
	RebuildAndFilterCategoryTree();
	return EActiveTimerReturnType::Stop;
}

#undef LOCTEXT_NAMESPACE
