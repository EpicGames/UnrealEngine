// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class STreeView;

class FPluginCategory;
class SPluginBrowser;

/**
 * Tree view that displays all of the plugin categories and allows the user to switch views
 */
class SPluginCategoryTree : public SCompoundWidget
{
public:

	enum class EFilterType : uint8
	{
		None,
		OnlyEnabled,
		OnlyDisabled,
	};

	SLATE_BEGIN_ARGS( SPluginCategoryTree )
	{
	}

	SLATE_END_ARGS()


	/** Widget constructor */
	void Construct( const FArguments& Args, const TSharedRef< class SPluginBrowser > Owner );

	/** Destructor */
	~SPluginCategoryTree();

	/** @return Gets the owner of this categories tree */
	SPluginBrowser& GetOwner();

	/** @return Returns the currently selected category item */
	TSharedPtr<FPluginCategory> GetSelectedCategory() const;

	/** Selects the specified category */
	void SelectCategory(const TSharedPtr<FPluginCategory>& CategoryToSelect);

	/** @return Returns true if the specified item is currently expanded in the tree */
	bool IsItemExpanded(const TSharedPtr<FPluginCategory> Item) const;

	/** Signal that the categories list needs to be refreshed */
	void SetNeedsRefresh();

	void ToggleFilterType(EFilterType FilterValue);
	bool IsFilterEnabled(EFilterType FilterValue) const;

private:
	
	/** Called to generate a widget for the specified tree item */
	TSharedRef<ITableRow> PluginCategoryTreeView_OnGenerateRow(TSharedPtr<FPluginCategory> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Given a tree item, fills an array of child items */
	void PluginCategoryTreeView_OnGetChildren(TSharedPtr<FPluginCategory> Item, TArray<TSharedPtr<FPluginCategory>>& OutChildren);

	/** Called when the user clicks on a plugin item, or when selection changes by some other means */
	void PluginCategoryTreeView_OnSelectionChanged(TSharedPtr<FPluginCategory> Item, ESelectInfo::Type SelectInfo);

	/** Rebuilds the category tree from scratch */
	void RebuildAndFilterCategoryTree();

	/** Callback for refreshing the tree */
	EActiveTimerReturnType TriggerCategoriesRefresh(double InCurrentTime, float InDeltaTime);

private:

	/** Weak pointer back to its owner */
	TWeakPtr< class SPluginBrowser > OwnerWeak;

	/** The tree view widget for our plugin categories tree */
	TSharedPtr<STreeView<TSharedPtr<FPluginCategory>>> TreeView;

	/** Root list of categories */
	TArray<TSharedPtr<FPluginCategory>> RootCategories;

	/** Category that includes all plugins */
	TSharedPtr<FPluginCategory> AllCategory;

	/** Category for built-in plugins */
	TSharedPtr<FPluginCategory> BuiltInCategory;

	/** Category for external plugins */
	TSharedPtr<FPluginCategory> ExternalCategory;

	/** Category for installed plugins */
	TSharedPtr<FPluginCategory> InstalledCategory;

	/** Category for project plugins */
	TSharedPtr<FPluginCategory> ProjectCategory;

	/** Category for mods */
	TSharedPtr<FPluginCategory> ModCategory;

	EFilterType FilterType;
};

