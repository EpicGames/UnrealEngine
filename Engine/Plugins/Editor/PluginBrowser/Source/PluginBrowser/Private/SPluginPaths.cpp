// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginPaths.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "IStructureDetailsView.h"
#include "Logging//StructuredLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "PluginBrowserModule.h"
#include "PropertyEditorModule.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SPluginPaths)

#define LOCTEXT_NAMESPACE "SPluginPaths"


namespace
{
	static const FName ProjectPathsName = GET_MEMBER_NAME_CHECKED(FPluginPaths_External, AdditionalPluginDirectories);
	static const FName UserPathsName = GET_MEMBER_NAME_CHECKED(FPluginPaths_External, UserPluginDirectories);
	static const FName CmdLinePathsName = GET_MEMBER_NAME_CHECKED(FPluginPaths_External, CommandLineDirectories);
	static const FName EnvironmentPathsName = GET_MEMBER_NAME_CHECKED(FPluginPaths_External, EnvironmentDirectories);
}


SLATE_IMPLEMENT_WIDGET(SPluginPaths)


void SPluginPaths::PrivateRegisterAttributes(FSlateAttributeInitializer& InAttributeInitializer)
{
}


void SPluginPaths::Construct(const FArguments& InArgs)
{
	ExternalPathsStruct = MakeShared<TStructOnScope<FPluginPaths_External>>();
	ExternalPathsStruct->InitializeAs<FPluginPaths_External>();
	FPluginPaths_External& ExternalPaths = GetExternalPaths();

	IPluginManager& PluginManager = IPluginManager::Get();

	// Categorize configured external paths.
	TSet<FExternalPluginPath> AllExternalPaths;
	PluginManager.GetExternalPluginSources(AllExternalPaths);

	for (const FExternalPluginPath& ExternalPath : AllExternalPaths)
	{
		switch (ExternalPath.Source)
		{
		case EPluginExternalSource::ProjectDescriptor:
			ExternalPaths.AdditionalPluginDirectories.Add(FDirectoryPath{ ExternalPath.Path });
			break;
		case EPluginExternalSource::Other:
			ExternalPaths.UserPluginDirectories.Add(FDirectoryPath{ ExternalPath.Path });
			break;
		case EPluginExternalSource::CommandLine:
			ExternalPaths.CommandLineDirectories.Add(FDirectoryPath{ ExternalPath.Path });
			break;
		case EPluginExternalSource::Environment:
			ExternalPaths.EnvironmentDirectories.Add(FDirectoryPath{ ExternalPath.Path });
			break;
		default:
			ensureAlways("Unhandled external plugin source");
		}
	}

	FPropertyEditorModule& PropertyEditorModule =
		FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;
	DetailsViewArgs.bShowHiddenPropertiesWhilePlayingOption = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsViewArgs.bShowKeyablePropertiesOption = false;
	DetailsViewArgs.bShowAnimatedPropertiesOption = false;

	// Setting these to true is necessary to specify our own Visible delegate,
	// which is otherwise overwritten by FStructureDetailsViewFilter.
	FStructureDetailsViewArgs StructViewArgs;
	StructViewArgs.bShowObjects = true;
	StructViewArgs.bShowInterfaces = true;

	ExternalView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructViewArgs, ExternalPathsStruct);
	ExternalView->GetDetailsView()->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SPluginPaths::HandleIsPropertyReadOnly));
	ExternalView->GetDetailsView()->SetIsPropertyVisibleDelegate(FIsPropertyReadOnly::CreateSP(this, &SPluginPaths::HandleIsPropertyVisible));
	ExternalView->GetDetailsView()->ForceRefresh();

	TSharedPtr<SWidget> CheckoutWidget;
	if (CanModifyProjectPaths())
	{
		CheckoutWidget = SNew(SSettingsEditorCheckoutNotice)
			.ConfigFilePath(FPaths::GetProjectFilePath());
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(24.f, 16.f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					// Category title
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("SettingsEditor.CatgoryAndSectionFont"))
					.Text(LOCTEXT("SettingsTitle", "External Plugin Directories"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 5.f, 0.f, 0.f)
				[
					// Category description
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(LOCTEXT("SettingsDescription", "Configure additional locations which should be enumerated for plugins."))
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 8.f, 0.f)
		[
			CheckoutWidget ? CheckoutWidget.ToSharedRef() : SNullWidget::NullWidget
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ExternalView->GetWidget().ToSharedRef()
		]
	];
}


void SPluginPaths::NotifyPreChange(class FEditPropertyChain* PropertyAboutToChange)
{
	FNotifyHook::NotifyPreChange(PropertyAboutToChange);

	// Update the cached "before" copy for subsequent diff in *post* change notification.
	PreviousExternalPathsForDiff = GetExternalPaths();
}

void SPluginPaths::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	FNotifyHook::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);

	if (InPropertyThatChanged == nullptr)
	{
		return;
	}

	const FName ActiveMemberName = InPropertyThatChanged->GetActiveMemberNode()->GetValue()->GetFName();

	bool bPluginDirectoriesChanged = false;

	if (ActiveMemberName == ProjectPathsName)
	{
		const FString ProjectPath = FPaths::GetProjectFilePath();
		if (!SettingsHelpers::CheckOutOrAddFile(ProjectPath))
		{
			SettingsHelpers::MakeWritable(ProjectPath);
		}

		TSet<FString> AddedPaths, RemovedPaths;
		DiffAddedAndRemoved(
			PreviousExternalPathsForDiff.AdditionalPluginDirectories,
			GetExternalPaths().AdditionalPluginDirectories,
			AddedPaths,
			RemovedPaths
		);

		if (AddedPaths.Num())
		{
			UE_LOGFMT(LogPluginBrowser, Display, "Added project plugin directories: {AddedPaths}",
				FString::Join(AddedPaths, TEXT("; ")));
		}

		if (RemovedPaths.Num())
		{
			UE_LOGFMT(LogPluginBrowser, Display, "Removed project plugin directories: {RemovedPaths}",
				FString::Join(RemovedPaths, TEXT("; ")));
		}

		IProjectManager& ProjectManager = IProjectManager::Get();
		for (const FString& RemovedPath : RemovedPaths)
		{
			bPluginDirectoriesChanged |= ProjectManager.UpdateAdditionalPluginDirectory(RemovedPath, false);
		}
		for (const FString& AddedPath : AddedPaths)
		{
			bPluginDirectoriesChanged |= ProjectManager.UpdateAdditionalPluginDirectory(AddedPath, true);
		}
	}
	else if (ActiveMemberName == UserPathsName)
	{
		TSet<FString> AddedPaths, RemovedPaths;
		DiffAddedAndRemoved(
			PreviousExternalPathsForDiff.UserPluginDirectories,
			GetExternalPaths().UserPluginDirectories,
			AddedPaths,
			RemovedPaths
		);

		if (AddedPaths.Num())
		{
			UE_LOGFMT(LogPluginBrowser, Display, "Added user plugin directories: {AddedPaths}",
				FString::Join(AddedPaths, TEXT("; ")));
		}

		if (RemovedPaths.Num())
		{
			UE_LOGFMT(LogPluginBrowser, Display, "Removed user plugin directories: {RemovedPaths}",
				FString::Join(RemovedPaths, TEXT("; ")));
		}

		IPluginManager& PluginManager = IPluginManager::Get();
		for (const FString& RemovedPath : RemovedPaths)
		{
			bPluginDirectoriesChanged |= PluginManager.RemovePluginSearchPath(RemovedPath, false);
		}
		for (const FString& AddedPath : AddedPaths)
		{
			bPluginDirectoriesChanged |= PluginManager.AddPluginSearchPath(AddedPath, false);
		}

		if (bPluginDirectoriesChanged)
		{
			PluginManager.RefreshPluginsList();
		}
	}

	if (bPluginDirectoriesChanged)
	{
		FPluginBrowserModule::Get().OnPluginDirectoriesChanged().Broadcast();
	}
}


FPluginPaths_External& SPluginPaths::GetExternalPaths()
{
	check(ExternalPathsStruct);
	return *ExternalPathsStruct->Get();
}


const FPluginPaths_External& SPluginPaths::GetExternalPaths() const
{
	check(ExternalPathsStruct);
	return *ExternalPathsStruct->Get();
}


bool SPluginPaths::CanModifyProjectPaths() const
{
	// If the new setting has been specified explicitly, use that.
	bool bCanModifyProject = true;
	if (GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanModifyProjectPluginDirectoriesFromBrowser"), bCanModifyProject, GEditorIni))
	{
		return bCanModifyProject;
	}

	// Fall back to whether plugins can be enabled/disabled in general.
	bool bModifyPluginsEnabled = true;
	GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanModifyPluginsFromBrowser"), bModifyPluginsEnabled, GEditorIni);
	return bModifyPluginsEnabled;
}


bool SPluginPaths::CanModifyUserPaths() const
{
	bool bCanModifyExternal = false;
	GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanModifyUserPluginDirectoriesFromBrowser"), bCanModifyExternal, GEditorIni);
	return bCanModifyExternal;
}


bool SPluginPaths::ShouldShowProjectPaths() const
{
	return CanModifyProjectPaths() || (GetExternalPaths().AdditionalPluginDirectories.Num() > 0);
}


bool SPluginPaths::ShouldShowUserPaths() const
{
	return CanModifyUserPaths() || (GetExternalPaths().UserPluginDirectories.Num() > 0);
}


bool SPluginPaths::HandleIsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent) const
{
	const FName PropertyName = InPropertyAndParent.Property.GetFName();
	const FName ParentName = (InPropertyAndParent.ParentProperties.Num() > 0)
		? InPropertyAndParent.ParentProperties[0]->GetFName() : NAME_None;

	if (PropertyName == ProjectPathsName || ParentName == ProjectPathsName)
	{
		return !CanModifyProjectPaths();
	}

	if (PropertyName == UserPathsName || ParentName == UserPathsName)
	{
		return !CanModifyUserPaths();
	}

	if (ParentName == CmdLinePathsName)
	{
		return true;
	}

	if (ParentName == EnvironmentPathsName)
	{
		return true;
	}

	return false;
}


bool SPluginPaths::HandleIsPropertyVisible(const FPropertyAndParent& InPropertyAndParent) const
{
	const FName PropertyName = InPropertyAndParent.Property.GetFName();
	const FName ParentName = (InPropertyAndParent.ParentProperties.Num() > 0)
		? InPropertyAndParent.ParentProperties[0]->GetFName() : NAME_None;

	if (PropertyName == ProjectPathsName || ParentName == ProjectPathsName)
	{
		return ShouldShowProjectPaths();
	}

	if (PropertyName == UserPathsName || ParentName == UserPathsName)
	{
		return ShouldShowUserPaths();
	}

	return true;
}


void SPluginPaths::DiffAddedAndRemoved(
	const TArray<FDirectoryPath>& InBefore,
	const TArray<FDirectoryPath>& InAfter,
	TSet<FString>& OutAdded,
	TSet<FString>& OutRemoved)
{
	TSet<FString> BeforeSet;
	BeforeSet.Reserve(InBefore.Num());
	for (const FDirectoryPath& BeforePath : InBefore)
	{
		if (!BeforePath.Path.IsEmpty())
		{
			BeforeSet.Add(BeforePath.Path);
		}
	}

	TSet<FString> AfterSet;
	AfterSet.Reserve(InAfter.Num());
	for (const FDirectoryPath& AfterPath : InAfter)
	{
		if (!AfterPath.Path.IsEmpty())
		{
			AfterSet.Add(AfterPath.Path);
		}
	}

	OutAdded = AfterSet.Difference(BeforeSet);
	OutRemoved = BeforeSet.Difference(AfterSet);
}


#undef LOCTEXT_NAMESPACE
