// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "UObject/StructOnScope.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SPluginPaths.generated.h"


class FEditPropertyChain;
struct FPropertyAndParent;
struct FPropertyChangedEvent;
class IStructureDetailsView;


/** Struct used as a model to expose relevant lists via details view. */
USTRUCT()
struct FPluginPaths_External
{
	GENERATED_BODY()

	UPROPERTY(Category="Plugin Directories", EditAnywhere, meta=(
		ToolTip="Stored in the .uproject descriptor."))
	TArray<FDirectoryPath> AdditionalPluginDirectories;

	UPROPERTY(Category="Plugin Directories", EditAnywhere)
	TArray<FDirectoryPath> UserPluginDirectories;

	UPROPERTY(Category="Plugin Directories", VisibleAnywhere, EditFixedSize, meta=(NoResetToDefault,
		ToolTip="Specified via the -PLUGIN= command line switch; cannot be modified here."))
	TArray<FDirectoryPath> CommandLineDirectories;

	UPROPERTY(Category="Plugin Directories", VisibleAnywhere, EditFixedSize, meta=(NoResetToDefault,
		ToolTip="Specified via the UE_ADDITIONAL_PLUGIN_PATHS environment variable; cannot be modified here."))
	TArray<FDirectoryPath> EnvironmentDirectories;
};


/** Widget that marshals details view to/from project and plugin manager. */
class SPluginPaths
	: public SCompoundWidget
	, public FNotifyHook
{
	SLATE_DECLARE_WIDGET(SPluginPaths, SCompoundWidget)

public:
	SLATE_BEGIN_ARGS(SPluginPaths) { }
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	//~ BEGIN FNotifyHook interface
	virtual void NotifyPreChange(FEditPropertyChain* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged) override;
	//~ END FNotifyHook interface

private:
	/** Mutable convenience getter for ExternalPathsStruct TStructOnScope. */
	FPluginPaths_External& GetExternalPaths();
	/** Const convenience getter for ExternalPathsStruct TStructOnScope. */
	const FPluginPaths_External& GetExternalPaths() const;

	bool CanModifyProjectPaths() const;
	bool CanModifyUserPaths() const;
	bool ShouldShowProjectPaths() const;
	bool ShouldShowUserPaths() const;

	/** Details view read-only delegate. */
	bool HandleIsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent) const;

	/** Details view visibility delegate. */
	bool HandleIsPropertyVisible(const FPropertyAndParent& InPropertyAndParent) const;

	/** Given two FDirectoryPath arrays, indicate what path strings were added/removed from InAfter compared to InBefore. */
	void DiffAddedAndRemoved(
		const TArray<FDirectoryPath>& InBefore,
		const TArray<FDirectoryPath>& InAfter,
		TSet<FString>& OutAdded,
		TSet<FString>& OutRemoved
	);

	/** The model being edited in the details view. */
	TSharedPtr<TStructOnScope<FPluginPaths_External>> ExternalPathsStruct;

	/** The primary editor widget. */
	TSharedPtr<IStructureDetailsView> ExternalView;

	/** Cache that's updated in NotifyPreChange, and diffed against in NotifyPostChange. */
	FPluginPaths_External PreviousExternalPathsForDiff;
};
