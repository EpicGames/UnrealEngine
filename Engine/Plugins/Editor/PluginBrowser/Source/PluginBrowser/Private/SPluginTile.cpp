// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginTile.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/App.h"
#include "Framework/Application/SlateApplication.h"
#include "PluginReferenceDescriptor.h"
#include "Widgets/Images/SImage.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Input/SCheckBox.h"
#include "Interfaces/IPluginManager.h"
#include "SPluginBrowser.h"
#include "PluginActions.h"
#include "PluginStyle.h"
#include "GameProjectGenerationModule.h"
#include "SPluginTileList.h"
#include "Widgets/Input/SHyperlink.h"
#include "Interfaces/IProjectManager.h"
#include "PluginBrowserModule.h"
#include "DesktopPlatformModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "PluginListTile"


void SPluginTile::Construct( const FArguments& Args, const TSharedRef<SPluginTileList> Owner, TSharedRef<IPlugin> InPlugin )
{
	OwnerWeak = Owner;
	Plugin = InPlugin;

	RecreateWidgets();
}

FText SPluginTile::GetPluginNameText() const
{
	return FText::FromString(Plugin->GetFriendlyName());
}

void SPluginTile::RecreateWidgets()
{
	const float PaddingAmount = FPluginStyle::Get()->GetFloat( "PluginTile.Padding" );
	const float ThumbnailImageSize = FPluginStyle::Get()->GetFloat( "PluginTile.ThumbnailImageSize" );
	const float HorizontalTilePadding = FPluginStyle::Get()->GetFloat("PluginTile.HorizontalTilePadding");
	const float VerticalTilePadding = FPluginStyle::Get()->GetFloat("PluginTile.VerticalTilePadding");

	// @todo plugedit: Also display whether plugin is editor-only, runtime-only, developer or a combination?
	//		-> Maybe a filter for this too?  (show only editor plugins, etc.)
	// @todo plugedit: Indicate whether plugin has content?  Filter to show only content plugins, and vice-versa?

	// @todo plugedit: Maybe we should do the FileExists check ONCE at plugin load time and not at query time

	const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();

	// Plugin thumbnail image
	FString Icon128FilePath = Plugin->GetBaseDir() / TEXT("Resources/Icon128.png");
	
	const FName BrushName(*Icon128FilePath);
	const FIntPoint Size = FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(BrushName);

	if ((Size.X > 0) && (Size.Y > 0))
	{
		PluginIconDynamicImageBrush = MakeShareable(new FSlateDynamicImageBrush(BrushName, FVector2D(Size.X, Size.Y)));
	}

	const bool bIsNewPlugin = FPluginBrowserModule::Get().IsNewlyInstalledPlugin(Plugin->GetName());

	// create support link
	TSharedPtr<SWidget> SupportWidget;
	{
		if (PluginDescriptor.SupportURL.IsEmpty() || !PluginDescriptor.SupportURL.StartsWith("https://"))
		{
			SupportWidget = SNullWidget::NullWidget;
		}
		else
		{
			FString SupportURL = PluginDescriptor.SupportURL;
			SupportWidget = SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Comment"))
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SHyperlink)
					.Text(LOCTEXT("SupportLink", "Support"))
					.ToolTipText(FText::Format(LOCTEXT("NavigateToSupportURL", "Open the plug-in's online support ({0})"), FText::FromString(SupportURL)))
					.OnNavigate_Lambda([=]() { FPlatformProcess::LaunchURL(*SupportURL, nullptr, nullptr); })
					.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
				];
		}
	}

	// create documentation link
	TSharedPtr<SWidget> DocumentationWidget;
	{
		if (PluginDescriptor.DocsURL.IsEmpty() || !PluginDescriptor.DocsURL.StartsWith("https://"))
		{
			DocumentationWidget = SNullWidget::NullWidget;
		}
		else
		{
			FString DocsURL = PluginDescriptor.DocsURL;
			DocumentationWidget = SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Documentation"))
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SHyperlink)
						.Text(LOCTEXT("DocumentationLink", "Documentation"))
						.ToolTipText(FText::Format(LOCTEXT("NavigateToDocumentation", "Open the plug-in's online documentation ({0})"), FText::FromString(DocsURL)))
						.OnNavigate_Lambda([=]() { FPlatformProcess::LaunchURL(*DocsURL, nullptr, nullptr); })
						.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
				];
		}
	}

	// create vendor link
	TSharedPtr<SWidget> CreatedByWidget;
	{
		if (PluginDescriptor.CreatedBy.IsEmpty())
		{
			CreatedByWidget = SNullWidget::NullWidget;
		}
		else if (PluginDescriptor.CreatedByURL.IsEmpty() || !PluginDescriptor.CreatedByURL.StartsWith("https://"))
		{
			CreatedByWidget = SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
						.Text(FText::FromString(PluginDescriptor.CreatedBy))
				];
		}
		else
		{
			FString CreatedByURL = PluginDescriptor.CreatedByURL;
			CreatedByWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[				
					SNew(SHyperlink)
						.Text(FText::FromString(PluginDescriptor.CreatedBy))
						.ToolTipText(FText::Format(LOCTEXT("NavigateToCreatedByURL", "Visit the vendor's web site ({0})"), FText::FromString(CreatedByURL)))
						.OnNavigate_Lambda([=]() { FPlatformProcess::LaunchURL(*CreatedByURL, nullptr, nullptr); })
						.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.OpenInBrowser"))
				];
		}
	}

	TSharedRef<SWidget> RestrictedPluginWidget = SNullWidget::NullWidget;
	if (FPaths::IsRestrictedPath(Plugin->GetDescriptorFileName()))
	{
		RestrictedPluginWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Bottom)
				.Padding(0.0f, 0.0f, 8.0f, 0.f)
				[
					SNew(SBorder)
					.BorderImage(FPluginStyle::Get()->GetBrush("PluginTile.RestrictedBorderImage"))
					.Padding(FMargin(8.f, 1.f, 8.f, 2.f))
					[
						SNew(STextBlock)
						.TextStyle(FPluginStyle::Get(), "PluginTile.BetaText")
						.Text(LOCTEXT("PluginRestrictedText", "Restricted"))
						.ToolTipText(FText::AsCultureInvariant(Plugin->GetDescriptorFileName()))
					]
					
				];
	}

	TSharedPtr<SWidget> ReferenceViewerWidget = SNullWidget::NullWidget;
	if (FPluginBrowserModule::Get().OnLaunchReferenceViewerDelegate().IsBound())
	{
		ReferenceViewerWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("ContentBrowser.ReferenceViewer"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SHyperlink)
				.OnNavigate(this, &SPluginTile::OnOpenPluginReferenceViewer)
				.Text(LOCTEXT("PluginReferenceViewer", "Reference Viewer"))
				.ToolTipText(LOCTEXT("NavigateToPluginReferenceViewer", "Launches the plugin reference viewer showing this plugins' references"))
				.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
			];
	}

	
	TSharedRef<SHorizontalBox> MiscLinks = SNew(SHorizontalBox);
	
	// support link
	MiscLinks->AddSlot()
		.Padding(PaddingAmount, PaddingAmount, PaddingAmount + 14.f, PaddingAmount)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPluginTile::GetAuthoringButtonsVisibility)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Edit"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SHyperlink)
				.OnNavigate(this, &SPluginTile::OnEditPlugin)
				.Text(LOCTEXT("EditPlugin", "Edit"))
				.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
			]
			
		];

	MiscLinks->AddSlot()
		.AutoWidth()
		.Padding(PaddingAmount, PaddingAmount, PaddingAmount + 14.f, PaddingAmount)
		.HAlign(HAlign_Left)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPluginTile::GetAuthoringButtonsVisibility)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Package"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SHyperlink)
				.OnNavigate(this, &SPluginTile::OnPackagePlugin)
				.Text(LOCTEXT("PackagePlugin", "Package"))
				.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
			]
		];

	if (ReferenceViewerWidget != SNullWidget::NullWidget)
	{
		MiscLinks->AddSlot()
			.AutoWidth()
			.Padding(PaddingAmount, PaddingAmount, PaddingAmount + 14.f, PaddingAmount)
			.HAlign(HAlign_Left)
			[
				ReferenceViewerWidget.ToSharedRef()
			];
	}

	if (DocumentationWidget != SNullWidget::NullWidget)
	{
		MiscLinks->AddSlot()
			.AutoWidth()
			.Padding(PaddingAmount, PaddingAmount, PaddingAmount + 14.f, PaddingAmount)
			.HAlign(HAlign_Left)
			[
				DocumentationWidget.ToSharedRef()
			];

	}

	if (SupportWidget != SNullWidget::NullWidget)
	{
		MiscLinks->AddSlot()
			.AutoWidth()
			.Padding(PaddingAmount, PaddingAmount, PaddingAmount + 14.f, PaddingAmount)
			.HAlign(HAlign_Left)
			[
				SupportWidget.ToSharedRef()
			];
	}
	
	ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(HorizontalTilePadding, VerticalTilePadding))
			[
				SNew(SBorder)
					.BorderImage(FPluginStyle::Get()->GetBrush("PluginTile.BorderImage"))
					.Padding(PaddingAmount)
					[
						SNew(SHorizontalBox)

						// Enable checkbox
						+ SHorizontalBox::Slot()
							.Padding(FMargin(18, 0, 17, 0))
							.HAlign(HAlign_Left)
							.AutoWidth()
							[
								SNew(SCheckBox)
									.OnCheckStateChanged(this, &SPluginTile::OnEnablePluginCheckboxChanged)
									.IsChecked(this, &SPluginTile::IsPluginEnabled)
									.IsEnabled(this, &SPluginTile::CanModifyPlugins)
									.ToolTipText(CanModifyPlugins() ?
										LOCTEXT("EnableDisableButtonToolTip", "Toggles whether this plugin is enabled for your current project.  You may need to restart the program for this change to take effect.")
										: LOCTEXT("NonEditableButtonToolTip", "Editing plugin enabled/disabled state from the Plugin Browser has been disabled for this project."))
							]
						// Thumbnail image
						+ SHorizontalBox::Slot()
							.Padding(PaddingAmount, PaddingAmount + 4.f, PaddingAmount + 10.f, PaddingAmount)
							.VAlign(VAlign_Top)
							.AutoWidth()
							[
								SNew(SBox)
								.WidthOverride(ThumbnailImageSize)
								.HeightOverride(ThumbnailImageSize)
								[
									SNew(SBorder)
									.BorderImage(FPluginStyle::Get()->GetBrush("PluginTile.ThumbnailBorderImage"))
									[
										SNew(SImage)
										.Image(PluginIconDynamicImageBrush.IsValid() ? PluginIconDynamicImageBrush.Get() : FPluginStyle::Get()->GetBrush("PluginTile.ThumbnailImageFallBack"))
									]
								]
								
							]

						+SHorizontalBox::Slot()
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SHorizontalBox)

										// Friendly name
										+SHorizontalBox::Slot()
											.AutoWidth()
											.VAlign(VAlign_Center)
											.Padding(PaddingAmount, PaddingAmount + 3.f, PaddingAmount + 8.f, 0.f)
											[
												SNew(STextBlock)
													.Text(GetPluginNameText())
													.HighlightText_Raw(&OwnerWeak.Pin()->GetOwner().GetPluginTextFilter(), &FPluginTextFilter::GetRawFilterText)
													.TextStyle(FPluginStyle::Get(), "PluginTile.NameText")
											]

										// "NEW!" label
										+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(0, PaddingAmount, bIsNewPlugin ? 8.f : 0.f, 0) // Extra Padding to the right if new label is visible
											.HAlign(HAlign_Left)
											.VAlign(VAlign_Center)
											[
												SNew(SBorder)
													.Padding(FMargin(8.f, 1.f, 8.f, 2.f))
													.BorderImage(FPluginStyle::Get()->GetBrush("PluginTile.NewLabelBorderImage"))
													[
														SNew(STextBlock)
															.Visibility(bIsNewPlugin ? EVisibility::Visible : EVisibility::Collapsed)
															.Text(LOCTEXT("PluginNewLabel", "New"))
															.TextStyle(FPluginStyle::Get(), "PluginTile.BetaText")
													]
											]
										// noredist/restricted label
										+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(0, PaddingAmount, 0, 0)
											.VAlign(VAlign_Center)
											[
												RestrictedPluginWidget
											]

										// beta version label
										+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(0, PaddingAmount, 0, 0)
											.VAlign(VAlign_Center)
											[

												SNew(SBorder)
												.BorderImage(FPluginStyle::Get()->GetBrush(PluginDescriptor.bIsExperimentalVersion ? "PluginTile.ExperimentalBorderImage" : "PluginTile.BetaBorderImage"))
												.Visibility((PluginDescriptor.bIsBetaVersion || PluginDescriptor.bIsExperimentalVersion) ? EVisibility::Visible : EVisibility::Collapsed)
												.Padding(FMargin(8.f, 1.f, 8.f, 2.f))
												[
													SNew(STextBlock)
													.TextStyle(FPluginStyle::Get(), "PluginTile.BetaText")
													.Text(PluginDescriptor.bIsExperimentalVersion ? LOCTEXT("PluginExperimentalVersionText", "Experimental") : LOCTEXT("PluginBetaVersionText", "Beta"))
													.ToolTipText(this, &SPluginTile::GetBetaOrExperimentalHelpText)
												]
											]

										// Gap
										+ SHorizontalBox::Slot()
											[
												SNew(SSpacer)
											]

										// Version
										+ SHorizontalBox::Slot()
											.HAlign(HAlign_Right)
											.Padding(PaddingAmount, PaddingAmount, PaddingAmount, 0)
											.AutoWidth()
											[
												SNew(SHorizontalBox)
																								
												// version number
												+ SHorizontalBox::Slot()
													.AutoWidth()
													.VAlign( VAlign_Bottom )
													.Padding(0.0f, 6.0f, 0.0f, 1.0f) // Lower padding to align font with version number base
													[
														SNew(STextBlock)
															.Text(LOCTEXT("PluginVersionLabel", "Version "))
															.TextStyle(FPluginStyle::Get(), "PluginTile.VersionNumberText")
													]

												+ SHorizontalBox::Slot()
													.AutoWidth()
													.VAlign( VAlign_Bottom )
													.Padding( 0.0f, 3.0f, 16.0f, 1.0f )	// Extra padding from the right edge
													[
														SNew(STextBlock)
															.Text(FText::FromString(PluginDescriptor.VersionName))
															.TextStyle(FPluginStyle::Get(), "PluginTile.VersionNumberText")
													]
											]
									]
			
								+ SVerticalBox::Slot()
									[
										SNew(SVerticalBox)
				
										// Description
										+ SVerticalBox::Slot()
											.Padding( PaddingAmount, 0, PaddingAmount, PaddingAmount)
											[
												SNew(SHorizontalBox)

												+SHorizontalBox::Slot()
												[
													SNew(STextBlock)
													.Text(FText::FromString(PluginDescriptor.Description))
													.HighlightText_Raw(&OwnerWeak.Pin()->GetOwner().GetPluginTextFilter(), &FPluginTextFilter::GetRawFilterText)
													.AutoWrapText(true)
												]
												+SHorizontalBox::Slot()
												.AutoWidth()
												.HAlign(HAlign_Right)
												.VAlign(VAlign_Top)
												.Padding(PaddingAmount + 14.f, 0.f, PaddingAmount + 14.f, 0.f)
												[
													CreatedByWidget.ToSharedRef()
												]
											]

										+ SVerticalBox::Slot()
											.Padding(PaddingAmount, PaddingAmount + 5.f, PaddingAmount, PaddingAmount + 4.f)
											.AutoHeight()
											[
												MiscLinks
											]
									]
							]
					]
			]
	];
}

bool SPluginTile::CanModifyPlugins() const
{
	bool bValue = true;
	GConfig->GetBool(TEXT("EditorSettings"), TEXT("bCanModifyPluginsFromBrowser"), bValue, GEditorIni);
	return bValue;
}

ECheckBoxState SPluginTile::IsPluginEnabled() const
{
	FPluginBrowserModule& PluginBrowserModule = FPluginBrowserModule::Get();
	if(PluginBrowserModule.HasPluginPendingEnable(Plugin->GetName()))
	{
		return PluginBrowserModule.GetPluginPendingEnableState(Plugin->GetName()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	else
	{
		return Plugin->IsEnabled()? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
}

void FindPluginDependencies(const FString& Name, TSet<FString>& Dependencies, TMap<FString, IPlugin*>& NameToPlugin)
{
	IPlugin* Plugin = NameToPlugin.FindRef(Name);
	if (Plugin != nullptr)
	{
		for (const FPluginReferenceDescriptor& Reference : Plugin->GetDescriptor().Plugins)
		{
			if (Reference.bEnabled && !Dependencies.Contains(Reference.Name))
			{
				Dependencies.Add(Reference.Name);
				FindPluginDependencies(Reference.Name, Dependencies, NameToPlugin);
			}
		}
	}
}

void SPluginTile::OnEnablePluginCheckboxChanged(ECheckBoxState NewCheckedState)
{
	if (!CanModifyPlugins())
	{
		return;
	}

	const bool bNewEnabledState = NewCheckedState == ECheckBoxState::Checked;

	const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();

	if (bNewEnabledState)
	{
		// If this is plugin is marked as beta, make sure the user is aware before enabling it.
		if (PluginDescriptor.bIsExperimentalVersion)
		{
			FText WarningMessage = FText::Format(LOCTEXT("Warning_EnablingExperimentalPlugin", "Plugin '{0}' is an experimental version. {1} Are you sure you want to enable the plugin?"), GetPluginNameText(), GetBetaOrExperimentalHelpText());
			if (EAppReturnType::No == FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage))
			{
				return;
			}
		}
		else if (PluginDescriptor.bIsBetaVersion)
		{
			FText WarningMessage = FText::Format(LOCTEXT("Warning_EnablingBetaPlugin", "Plugin '{0}' is a beta version. {1} Are you sure you want to enable the plugin?"), GetPluginNameText(), GetBetaOrExperimentalHelpText());
			if (EAppReturnType::No == FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage))
			{
				return;
			}
		}
	}
	else
	{
		// Get all the plugins we know about
		TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPlugins();

		// Build a map of plugin by name
		TMap<FString, IPlugin*> NameToPlugin;
		for (TSharedRef<IPlugin>& EnabledPlugin : EnabledPlugins)
		{
			NameToPlugin.FindOrAdd(EnabledPlugin->GetName()) = &(EnabledPlugin.Get());
		}

		// Find all the plugins which are dependent on this plugin
		TArray<FString> DependentPluginNames;
		for (TSharedRef<IPlugin>& EnabledPlugin : EnabledPlugins)
		{
			FString EnabledPluginName = EnabledPlugin->GetName();

			TSet<FString> Dependencies;
			FindPluginDependencies(EnabledPluginName, Dependencies, NameToPlugin);

			if (Dependencies.Num() > 0 && Dependencies.Contains(Plugin->GetName()))
			{
				FText Caption = LOCTEXT("DisableDependenciesCaption", "Disable Dependencies");
				FText Message = FText::Format(LOCTEXT("DisableDependenciesMessage", "This plugin is required by {0}. Would you like to disable it as well?"), FText::FromString(EnabledPluginName));
				if (FMessageDialog::Open(EAppMsgType::YesNo, Message, Caption) == EAppReturnType::No)
				{
					return;
				}
				DependentPluginNames.Add(EnabledPluginName);
			}
		}

		// Disable all the dependent plugins too
		for (const FString& DependentPluginName : DependentPluginNames)
		{
			FText FailureMessage;
			if (!IProjectManager::Get().SetPluginEnabled(DependentPluginName, false, FailureMessage))
			{
				FMessageDialog::Open(EAppMsgType::Ok, FailureMessage);
			}

			TSharedPtr<IPlugin> DependentPlugin = IPluginManager::Get().FindPlugin(DependentPluginName);
			if (DependentPlugin.IsValid())
			{
				FPluginBrowserModule::Get().SetPluginPendingEnableState(DependentPluginName, DependentPlugin->IsEnabled(), false);
			}
		}
	}

	// Finally, enable/disable the plugin we selected
	FText FailMessage;
	bool bSuccess = IProjectManager::Get().SetPluginEnabled(Plugin->GetName(), bNewEnabledState, FailMessage);

	if (bSuccess && IProjectManager::Get().IsCurrentProjectDirty())
	{
		FGameProjectGenerationModule::Get().TryMakeProjectFileWriteable(FPaths::GetProjectFilePath());
		bSuccess = IProjectManager::Get().SaveCurrentProjectToDisk(FailMessage);
	}

	if (bSuccess)
	{
		FPluginBrowserModule::Get().SetPluginPendingEnableState(Plugin->GetName(), Plugin->IsEnabled(), bNewEnabledState);
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, FailMessage);
	}
}

EVisibility SPluginTile::GetAuthoringButtonsVisibility() const
{
	if (FApp::IsEngineInstalled() && Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
	{
		return EVisibility::Collapsed;
	}
	if (FApp::IsInstalled() && Plugin->GetType() != EPluginType::Mod)
	{
		return EVisibility::Collapsed;
	}
	return EVisibility::Visible;
}

void SPluginTile::OnEditPlugin()
{
	FPluginBrowserModule::Get().OpenPluginEditor(Plugin.ToSharedRef(), OwnerWeak.Pin(), FSimpleDelegate::CreateRaw(this, &SPluginTile::OnEditPluginFinished));
}

void SPluginTile::OnEditPluginFinished()
{
	// Recreate the widgets on this tile
	RecreateWidgets();

	// Refresh the parent too
	if(OwnerWeak.IsValid())
	{
		OwnerWeak.Pin()->GetOwner().SetNeedsRefresh();
	}
}

void SPluginTile::OnPackagePlugin()
{
	if (ensure(Plugin.IsValid()))
	{
		PluginActions::PackagePlugin(Plugin.ToSharedRef(), AsShared());
	}
}

void SPluginTile::OnOpenPluginReferenceViewer()
{
	FPluginBrowserModule::Get().OnLaunchReferenceViewerDelegate().Execute(Plugin);
}

FText SPluginTile::GetBetaOrExperimentalHelpText() const
{
	if (!Plugin)
	{
		return FText();
	}

	const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();
	if (PluginDescriptor.bIsExperimentalVersion)
	{
		return LOCTEXT("Description_ExperimentalPlugin", "Epic does not recommend shipping projects with experimental plugins. APIs, features, and the plugin itself are subject to change or be removed without notice.");
	}
	if (PluginDescriptor.bIsBetaVersion)
	{
		return LOCTEXT("Description_BetaPlugin", "Epic recommends using caution when shipping projects with beta plugins. Beta plugins support backwards compatibility for assets and APIs, but performance, stability, and platform support may not be shipping quality.");
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
