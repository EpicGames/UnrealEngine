// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

enum class ECheckBoxState : uint8;
struct FSlateDynamicImageBrush;

class IPlugin;
class UPluginMetadataObject;

/**
 * Widget that represents a "tile" for a single plugin in our plugins list
 */
class SPluginTile : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SPluginTile )
	{
	}

	SLATE_END_ARGS()


	/** Widget constructor */
	void Construct( const FArguments& Args, const TSharedRef< class SPluginTileList > Owner, TSharedRef<IPlugin> Plugin );

private:

	/** Returns text to display for the plugin name. */
	FText GetPluginNameText() const;

	/** Updates the contents of this tile */
	void RecreateWidgets();

	/** Returns whether users are able to modify the enabled/disabled state of plugins from the plugin browser */
	bool CanModifyPlugins() const;

	/** Returns the checked state for the enabled checkbox */
	ECheckBoxState IsPluginEnabled() const;

	/** Called when the enabled checkbox is clicked */
	void OnEnablePluginCheckboxChanged(ECheckBoxState NewCheckedState);

	/** Used to determine whether to show the edit and package buttons for this plugin */
	EVisibility GetAuthoringButtonsVisibility() const;

	/** Called when the 'edit' hyperlink is clicked */
	void OnEditPlugin();

	/** Called when the edit window is committed */
	void OnEditPluginFinished();

	/** Called when the 'package' hyperlink is clicked */
	void OnPackagePlugin();

	/** Called when the "reference viewer' hyperlink is clicked */
	void OnOpenPluginReferenceViewer();

	/** Called when the beta or experimental flags are hovered over, or when a beta or experimental plugin is enabled */
	FText GetBetaOrExperimentalHelpText() const;

private:

	/** The item we're representing the in tree */
	TSharedPtr<IPlugin> Plugin;

	/** Weak pointer back to its owner */
	TWeakPtr< class SPluginTileList > OwnerWeak;

	/** Brush resource for the image that is dynamically loaded */
	TSharedPtr< FSlateDynamicImageBrush > PluginIconDynamicImageBrush;
};

