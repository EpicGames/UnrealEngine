// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginTileList.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Views/SListView.h"
#include "SPluginBrowser.h"
#include "SPluginCategory.h"
#include "SPluginTile.h"
#include "Interfaces/IPluginManager.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "PluginBrowserModule.h"


#define LOCTEXT_NAMESPACE "PluginList"


void SPluginTileList::Construct( const FArguments& Args, const TSharedRef< SPluginBrowser > Owner )
{
	OwnerWeak = Owner;

	// Find out when the plugin text filter changes
	Owner->GetPluginTextFilter().OnChanged().AddSP( this, &SPluginTileList::OnPluginTextFilterChanged );

	bIsActiveTimerRegistered = false;
	RebuildAndFilterPluginList();

	// @todo plugedit: Have optional compact version with only plugin icon + name + version?  Only expand selected?

	PluginListView =
		SNew( SListView<TSharedRef<IPlugin>> )
		.SelectionMode( ESelectionMode::None )		// No need to select plugins!
		.ListItemsSource( &PluginListItems )
		.OnGenerateRow( this, &SPluginTileList::PluginListView_OnGenerateRow )
		.ListViewStyle( FAppStyle::Get(), "SimpleListView" );

	TSharedPtr<SScrollBorder> ChildWidget = SNew(SScrollBorder, PluginListView.ToSharedRef())
		.BorderFadeDistance(this, &SPluginTileList::GetListBorderFadeDistance)
		[
			PluginListView.ToSharedRef()
		];

	ChildSlot.AttachWidget(ChildWidget.ToSharedRef());
}


SPluginTileList::~SPluginTileList()
{
	const TSharedPtr< SPluginBrowser > Owner( OwnerWeak.Pin() );
	if( Owner.IsValid() )
	{
		Owner->GetPluginTextFilter().OnChanged().RemoveAll( this );
	}
}


/** @return Gets the owner of this list */
SPluginBrowser& SPluginTileList::GetOwner()
{
	return *OwnerWeak.Pin();
}


TSharedRef<ITableRow> SPluginTileList::PluginListView_OnGenerateRow( TSharedRef<IPlugin> Item, const TSharedRef<STableViewBase>& OwnerTable )
{
	return
		SNew( STableRow< TSharedRef<IPlugin> >, OwnerTable )
		[
			SNew( SPluginTile, SharedThis( this ), Item )
		];
}



void SPluginTileList::RebuildAndFilterPluginList()
{
	// Build up the initial list of plugins
	{
		PluginListItems.Reset();

		// Get the currently selected category
		const TSharedPtr<FPluginCategory>& SelectedCategory = OwnerWeak.Pin()->GetSelectedCategory();
		if( SelectedCategory.IsValid() )
		{
			for(TSharedRef<IPlugin> Plugin: SelectedCategory->Plugins)
			{
				if(OwnerWeak.Pin()->GetPluginTextFilter().PassesFilter(&Plugin.Get()))
				{
					PluginListItems.Add(Plugin);
				}
			}
		}

		// Sort the plugins alphabetically
		{
			struct FPluginListItemSorter
			{
				bool operator()(const TSharedRef<IPlugin>& A, const TSharedRef<IPlugin>& B) const
				{
					return A->GetFriendlyName() < B->GetFriendlyName();
				}
			};
			PluginListItems.Sort( FPluginListItemSorter() );
		}
	}


	// Update the list widget
	if( PluginListView.IsValid() )
	{
		PluginListView->RequestListRefresh();
	}
}

EActiveTimerReturnType SPluginTileList::TriggerListRebuild(double InCurrentTime, float InDeltaTime)
{
	RebuildAndFilterPluginList();

	bIsActiveTimerRegistered = false;
	return EActiveTimerReturnType::Stop;
}

void SPluginTileList::OnPluginTextFilterChanged()
{
	SetNeedsRefresh();
}


void SPluginTileList::SetNeedsRefresh()
{
	if (!bIsActiveTimerRegistered)
	{
		bIsActiveTimerRegistered = true;
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SPluginTileList::TriggerListRebuild));
	}
}

FVector2D SPluginTileList::GetListBorderFadeDistance() const
{
	// Negative fade distance when there is no restart editor warning to make the shadow disappear
	FVector2D ReturnVal = FPluginBrowserModule::Get().ShowPendingRestart() ? FVector2D(0.01f, 0.01f) : FVector2D(-1.0f, -1.0f);

	return ReturnVal;
}

#undef LOCTEXT_NAMESPACE
