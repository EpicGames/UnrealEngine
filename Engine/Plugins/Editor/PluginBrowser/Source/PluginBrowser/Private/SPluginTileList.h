// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
template <typename ItemType> class SListView;

class IPlugin;
class SPluginBrowser;

/**
 * A filtered list of plugins, driven by a category selection
 */
class SPluginTileList : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SPluginTileList )
	{
	}

	SLATE_END_ARGS()

	/** Widget constructor */
	void Construct( const FArguments& Args, const TSharedRef< class SPluginBrowser > Owner );

	/** Destructor */
	~SPluginTileList();

	/** @return Gets the owner of this list */
	SPluginBrowser& GetOwner();

	/** Called to invalidate the list */
	void SetNeedsRefresh();


private:

	/** Called when the plugin text filter has changed what its filtering */
	void OnPluginTextFilterChanged();

	/** Called to generate a widget for the specified list item */
	TSharedRef<ITableRow> PluginListView_OnGenerateRow(TSharedRef<IPlugin> Item, const TSharedRef<STableViewBase>& OwnerTable );

	/** Rebuilds the list of plugins from scratch and applies filtering. */
	void RebuildAndFilterPluginList();

	/** One-off active timer to trigger a full refresh when something has changed with either our filtering or the loaded plugin set */
	EActiveTimerReturnType TriggerListRebuild(double InCurrentTime, float InDeltaTime);

	FVector2D GetListBorderFadeDistance() const;
private:

	/** Weak pointer back to its owner */
	TWeakPtr< class SPluginBrowser > OwnerWeak;

	/** The list view widget for our plugins list */
	TSharedPtr<SListView<TSharedRef<IPlugin>>> PluginListView;

	/** List of everything that we want to display in the plugin list */
	TArray<TSharedRef<IPlugin>> PluginListItems;

	/** Whether the active timer to refresh the list is registered */
	bool bIsActiveTimerRegistered;
};

