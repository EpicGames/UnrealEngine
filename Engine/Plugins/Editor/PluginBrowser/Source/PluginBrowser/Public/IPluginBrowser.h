// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Features/IPluginsEditorFeature.h"

DECLARE_DELEGATE_OneParam(FOnLaunchReferenceViewer, TSharedPtr<IPlugin>);
DECLARE_MULTICAST_DELEGATE(FOnPluginDirectoriesChanged);

/**
 * The public interface to this module
 */
class IPluginBrowser : public IModuleInterface, public IPluginsEditorFeature
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPluginBrowser& Get()
	{
		return FModuleManager::LoadModuleChecked< IPluginBrowser >( "PluginBrowser" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "PluginBrowser" );
	}

public:
	/**
	 * Allows other modules to spawn the new plugin interface with their own definition
	 * 
	 * @param	SpawnTabArgs			Arguments for spawning the new plugin tab
	 * @param	PluginWizardDefinition	The definition that drives the functionality of the new plugin tab
	 * @return	A shared reference to the dock tab where the new plugin widget is created.
	 */
	virtual TSharedRef<class SDockTab> SpawnPluginCreatorTab(const class FSpawnTabArgs& SpawnTabArgs, TSharedPtr<class IPluginWizardDefinition> PluginWizardDefinition) = 0;

	/**
	 * Allows another module that supplies the plugin reference viewer to launch when requested.
	 */
	virtual FOnLaunchReferenceViewer& OnLaunchReferenceViewerDelegate() = 0;

	/** Called when the external plugin directories configuration is modified via the browser. */
	virtual FOnPluginDirectoriesChanged& OnPluginDirectoriesChanged() = 0;

	/** Can optionally be bound to override the "restart required" button behavior. */
	virtual FSimpleDelegate& OnRestartClicked() = 0;
};
