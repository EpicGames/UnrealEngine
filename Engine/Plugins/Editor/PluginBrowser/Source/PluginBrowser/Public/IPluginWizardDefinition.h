// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModuleDescriptor.h"

enum class ECheckBoxState : uint8;
namespace ESelectInfo { enum Type : int; }
struct FPluginTemplateDescription;

class IPluginWizardDefinition : public TSharedFromThis<IPluginWizardDefinition>
{
public:

	virtual ~IPluginWizardDefinition() = default;

	/** Returns the plugin templates available to this definition */
	virtual const TArray<TSharedRef<FPluginTemplateDescription>>& GetTemplatesSource() const = 0;

	/** Changes the selection internally to match the supplied items */
	virtual void OnTemplateSelectionChanged(TSharedPtr<FPluginTemplateDescription> InSelectedItem, ESelectInfo::Type SelectInfo) = 0;

	/** Returns true if the definition has a valid template selection */
	virtual bool HasValidTemplateSelection() const = 0;

	/** Returns the currently selected template */
	virtual TSharedPtr<FPluginTemplateDescription> GetSelectedTemplate() const = 0;

	/** Clears the template selection */
	virtual void ClearTemplateSelection() = 0;

	/** Returns true if the wizard for this definition can show on startup */
	virtual bool CanShowOnStartup() const = 0;

	/** Returns true if the selected template will generate code */
	virtual bool HasModules() const = 0;

	/** Returns true if the plugin is a mod */
	virtual bool IsMod() const = 0;

	/** Callback for when the 'Show on Startup' checkbox changes in the plugin wizard. Only used if the definition allows for game mod plugins */
	virtual void OnShowOnStartupCheckboxChanged(ECheckBoxState CheckBoxState) = 0;

	/** Gets the state of the 'Show on Startup' checkbox. Only used if the definition allows for game mod plugins */
	virtual ECheckBoxState GetShowOnStartupCheckBoxState() const = 0;

	/** Returns a custom header widget for the new plugin wizard, if desired. */
	virtual TSharedPtr<class SWidget> GetCustomHeaderWidget() = 0;

	/** Gets the instructions to be shown when creating a new plugin */
	virtual FText GetInstructions() const = 0;

	/** Gets the icon path for the current template selection. Returns true if the plugin requires a default icon */
	virtual bool GetPluginIconPath(FString& OutIconPath) const = 0;

	/** Gets the ModuleDescriptor for the plugin based on the selection */
	virtual EHostType::Type GetPluginModuleDescriptor() const = 0;

	/** Gets the LoadingPhase for the plugin based on the selection */
	virtual ELoadingPhase::Type GetPluginLoadingPhase() const = 0;

	/** Gets the icon path for the specified template. Returns true if it requires a default icon */
	virtual bool GetTemplateIconPath(TSharedRef<FPluginTemplateDescription> InTemplate, FString& OutIconPath) const = 0;

	/** Gets the folder path of the current template selection */
	virtual FString GetPluginFolderPath() const = 0;

	/** Gets the folders for the current template selection */
	virtual TArray<FString> GetFoldersForSelection() const = 0;

	/** Called when a plugin is created, with a bool indicating whether creation was actually successful. */
	virtual void PluginCreated(const FString& PluginName, bool bWasSuccessful) const = 0;
};
